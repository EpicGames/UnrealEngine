// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Platform.ush"

#if !defined(SUPPORTS_BINDLESS_UNIFORM_BUFFER)
	#define SUPPORTS_BINDLESS_UNIFORM_BUFFER 0

#elif !PLATFORM_SUPPORTS_CONSTANTBUFFER_OBJECT
	// Bindless Uniform Buffers require ConstantBufer<> support
	// TODO: christopher.waters - should this be required at a higher level?

	#undef SUPPORTS_BINDLESS_UNIFORM_BUFFER
	#define SUPPORTS_BINDLESS_UNIFORM_BUFFER 0
#endif

#if UE_BINDLESS_ENABLED
	#if !defined(UE_BINDLESS_SRV_HEAP_DECL)
		#define UE_BINDLESS_SRV_HEAP_DECL(Type)
	#endif

	#if !defined(UE_BINDLESS_UAV_HEAP_DECL)
		#define UE_BINDLESS_UAV_HEAP_DECL(Type)
	#endif

	#if !defined(UE_BINDLESS_CBV_HEAP_DECL)
		#define UE_BINDLESS_CBV_HEAP_DECL(Type)
	#endif

	#if !defined(UE_BINDLESS_SAMPLER_HEAP_DECL)
		#define UE_BINDLESS_SAMPLER_HEAP_DECL(Type)
	#endif

	#if SUPPORTS_BINDLESS_UNIFORM_BUFFER
		#define UB_SAFE_TYPE(UniformBufferName) SafeType##UniformBufferName##Constants

		#define UB_BINDLESS_DECL(UniformBufferType, UniformBufferName) \
			typedef ConstantBuffer<UniformBufferType> UB_SAFE_TYPE(UniformBufferName); \
			UE_BINDLESS_CBV_HEAP_DECL(UB_SAFE_TYPE(UniformBufferName)); \
			UB_SAFE_TYPE(UniformBufferName) Get##UniformBufferName##Constants(uint Index) { return GetCBVFromHeap(UB_SAFE_TYPE(UniformBufferName), Index); }

	#endif // SUPPORTS_BINDLESS_UNIFORM_BUFFER

	#define UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember) SafeType##UniformBufferName##_##UniformBufferMember

	// Declarations for UB member types and heaps.
	#define UB_RES_DECL_SRV(Type, UniformBufferName, UniformBufferMember) \
		typedef Type UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember); \
		UE_BINDLESS_SRV_HEAP_DECL(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember))

	#define UB_RES_DECL_UAV(Type, UniformBufferName, UniformBufferMember) \
		typedef Type UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember); \
		UE_BINDLESS_UAV_HEAP_DECL(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember))

	#define UB_RES_DECL_SAMPLER(Type, UniformBufferName, UniformBufferMember) \
		typedef Type UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember); \
		UE_BINDLESS_SAMPLER_HEAP_DECL(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember))

	// Getters for cbuffer members
	#define UB_CB_LOAD_SRV(UniformBufferName, UniformBufferMember) \
		GetSRVFromHeap(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember), UB_CB_PREFIXED_MEMBER_ACCESS(UniformBufferName, BindlessSRV_, UniformBufferMember))

	#define UB_CB_LOAD_UAV(UniformBufferName, UniformBufferMember) \
		GetUAVFromHeap(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember), UB_CB_PREFIXED_MEMBER_ACCESS(UniformBufferName, BindlessUAV_, UniformBufferMember))

	#define UB_CB_LOAD_SAMPLER(UniformBufferName, UniformBufferMember) \
		GetSamplerFromHeap(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember), UB_CB_PREFIXED_MEMBER_ACCESS(UniformBufferName, BindlessSampler_, UniformBufferMember))

	// Getters for ConstantBuffer<> members. The constant buffer variable name needs to be an argument because it could be a local variable.
	#define UB_CBO_LOAD_SRV(UniformBufferName, VarName, UniformBufferMember) \
		GetSRVFromHeap(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember), VarName##.BindlessSRV_##UniformBufferMember)

	#define UB_CBO_LOAD_UAV(UniformBufferName, VarName, UniformBufferMember) \
		GetUAVFromHeap(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember), VarName##.BindlessUAV_##UniformBufferMember)

	#define UB_CBO_LOAD_SAMPLER(UniformBufferName, VarName, UniformBufferMember) \
		GetSamplerFromHeap(UB_RES_SAFE_TYPE(UniformBufferName, UniformBufferMember), VarName##.BindlessSampler_##UniformBufferMember)

	// Used in standard resource definitions, note that they are all cbuffer members
	#define UB_RESOURCE_MEMBER_SRV(Type, UBName, Name) \
		UB_RES_SAFE_TYPE(UBName, Name) GetBindlessResource##UBName##_##Name() { return UB_CB_LOAD_SRV(UBName, Name); } \
		static const UB_RES_SAFE_TYPE(UBName, Name) UBName##_##Name = GetBindlessResource##UBName##_##Name()

	#define UB_RESOURCE_MEMBER_UAV(Type, UBName, Name) \
		UB_RES_SAFE_TYPE(UBName, Name) GetBindlessResource##UBName##_##Name() { return UB_CB_LOAD_UAV(UBName, Name); } \
		static const UB_RES_SAFE_TYPE(UBName, Name) UBName##_##Name = GetBindlessResource##UBName##_##Name()

	#define UB_RESOURCE_MEMBER_SAMPLER(Type, UBName, Name) \
		UB_RES_SAFE_TYPE(UBName, Name) GetBindlessSampler##UBName##_##Name() { return UB_CB_LOAD_SAMPLER(UBName, Name); } \
		static const UB_RES_SAFE_TYPE(UBName, Name) UBName##_##Name = GetBindlessSampler##UBName##_##Name()

#else
	#define UB_RES_DECL_SRV(Type, UniformBufferName, UniformBufferMember)
	#define UB_RES_DECL_UAV(Type, UniformBufferName, UniformBufferMember)
	#define UB_RES_DECL_SAMPLER(Type, UniformBufferName, UniformBufferMember)

	#define UB_RESOURCE_MEMBER_SRV(Type, UBName, Name) Type UBName##_##Name
	#define UB_RESOURCE_MEMBER_UAV(Type, UBName, Name) Type UBName##_##Name
	#define UB_RESOURCE_MEMBER_SAMPLER(Type, UBName, Name) Type UBName##_##Name
#endif

//
// Descriptor Range support - Experimental feature, this code may completely change in the future.
#if UE_BINDLESS_ENABLED && PLATFORM_SUPPORTS_DESCRIPTOR_RANGE

	// .x = StartIndex
	// .y = Count
	typedef uint2 FDescriptorRange;

	uint GetDescriptorRangeStartIndex(FDescriptorRange InDescriptorRange) { return InDescriptorRange.x; }
	uint GetDescriptorRangeCount     (FDescriptorRange InDescriptorRange) { return InDescriptorRange.y; }

	#define GetDescriptorRangeSRV(InType, InDescriptorRange, InIndex) GetSRVFromHeap(InType, GetDescriptorRangeStartIndex(InDescriptorRange) + InIndex)
	#define GetDescriptorRangeUAV(InType, InDescriptorRange, InIndex) GetUAVFromHeap(InType, GetDescriptorRangeStartIndex(InDescriptorRange) + InIndex)

	// Declares the heap array for a given resource type so it can be indexed via GetDescriptorRangeSRV/UAV.
	// Must be called once per type used with a descriptor range, at global scope in the shader.
	// On Vulkan this expands to the per-type unbounded array declaration; on other platforms it is a no-op.
	#define DESCRIPTOR_RANGE_HEAP_DECL_SRV(Type) UE_BINDLESS_SRV_HEAP_DECL(Type)
	#define DESCRIPTOR_RANGE_HEAP_DECL_UAV(Type) UE_BINDLESS_UAV_HEAP_DECL(Type)
#endif

//
// Resource Collection support

#if UE_BINDLESS_ENABLED

typedef ByteAddressBuffer FResourceCollection;

uint GetCountFromResourceCollection(FResourceCollection ResourceCollection)
{
	return ResourceCollection.Load(0);
}

uint GetResourceIndexFromResourceCollection(FResourceCollection ResourceCollection, uint CollectionIndex)
{
	int Offset = (CollectionIndex + 1) * 4;
	return ResourceCollection.Load(Offset);
}

#if defined(BINDLESS_SRV_ARRAY_PREFIX)
	#define RESOURCE_COLLECTION_HEAP_NAME_INNER1(Prefix, Name) Prefix##Name
	#define RESOURCE_COLLECTION_HEAP_NAME_INNER(Prefix, Name) RESOURCE_COLLECTION_HEAP_NAME_INNER1(Prefix,Name)
	#define RESOURCE_COLLECTION_HEAP_NAME(Name) RESOURCE_COLLECTION_HEAP_NAME_INNER(BINDLESS_SRV_ARRAY_PREFIX, Name)

	#define TEXTURE_FROM_COLLECTION_T(TType) \
		typedef TType SafeType##TType; \
		SafeType##TType RESOURCE_COLLECTION_HEAP_NAME(TType)[]; \
		SafeType##TType TextureFromCollection_##TType(FResourceCollection ResourceCollection, uint CollectionIndex) \
		{ \
			uint ResourceIndex = GetResourceIndexFromResourceCollection(ResourceCollection, CollectionIndex); \
			return RESOURCE_COLLECTION_HEAP_NAME(TType)[ResourceIndex]; \
		}

	#define BUFFER_FROM_COLLECTION_T(TType) \
		typedef TType SafeType##TType; \
		SafeType##TType RESOURCE_COLLECTION_HEAP_NAME(TType)[]; \
		SafeType##TType BufferFromCollection_##TType(FResourceCollection ResourceCollection, uint CollectionIndex) \
		{ \
			uint ResourceIndex = GetResourceIndexFromResourceCollection(ResourceCollection, CollectionIndex); \
			return RESOURCE_COLLECTION_HEAP_NAME(TType)[ResourceIndex]; \
		}
#else
	#define TEXTURE_FROM_COLLECTION_T(TType) \
		TType TextureFromCollection_##TType(FResourceCollection ResourceCollection, uint CollectionIndex) \
		{ \
			uint ResourceIndex = GetResourceIndexFromResourceCollection(ResourceCollection, CollectionIndex); \
			return GetSRVFromHeap(TType, ResourceIndex); \
		}

	#define BUFFER_FROM_COLLECTION_T(TType) \
		TType BufferFromCollection_##TType(FResourceCollection ResourceCollection, uint CollectionIndex) \
		{ \
			uint ResourceIndex = GetResourceIndexFromResourceCollection(ResourceCollection, CollectionIndex); \
			return GetSRVFromHeap(TType, ResourceIndex); \
		}
#endif

TEXTURE_FROM_COLLECTION_T(Texture2D)
TEXTURE_FROM_COLLECTION_T(TextureCube)
TEXTURE_FROM_COLLECTION_T(Texture2DArray)
TEXTURE_FROM_COLLECTION_T(TextureCubeArray)
TEXTURE_FROM_COLLECTION_T(Texture3D)

typedef Buffer<float2> Buffer_float2;
BUFFER_FROM_COLLECTION_T(Buffer_float2)

#endif

#if defined(INCLUDE_PLATFORM_BINDLESS_RESOURCES_USH)
	#include "/Platform/Public/PlatformBindlessResources.ush"
#elif SM6_PROFILE
	#include "Platform/D3D/PlatformBindlessResources.ush"
#endif
