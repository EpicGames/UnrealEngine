// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DoublePixelVectorization.ush: API to vectorize a lane to process 2 pixel
	instead of one, to take full advantage of v_pk_*_[iuf]16 instructions
=============================================================================*/
  
#pragma once

#include "Platform.ush"

#define PLATFORM_SUPPORT_16BITS_DPV (PLATFORM_SUPPORTS_REAL_TYPES)

// Dual Pixel vectoring processes two pixel per lane.
#define DPV_PIXEL_PER_LANE 2


// ------------------------------------------------- DEFINE_DPV_FUNCTION_FOR_ALL_32BITS

// Helper to get the type for a single pixel vector
#define __DPV_VECTOR(TYPE, COMPONENTS) \
	TYPE##COMPONENTS

// Helper to get the type for a dual pixel scalar
#define __DPV_MULTI_SCALAR(TYPE) \
	TYPE##2

// Helper to get the type for a dual pixel vector
#define __DPV_MULTI_VECTOR(TYPE, COMPONENTS) \
	TYPE##COMPONENTS##x2

// Helper to get the transposed type for a dual pixel vector
#define __DPV_MULTI_VECTOR_TRANSPOSED(TYPE, COMPONENTS) \
	TYPE##2x##COMPONENTS


// __DPV_FOREACH_COMPONENTS() helper to write per component code with abritrary separator (like an operator for dot product)
#define __DPV_FOREACH_COMPONENTS_2(COMPONENT_CODE, COMPONENT_SEPARATOR, ...) \
	COMPONENT_CODE(0, __VA_ARGS__) \
	COMPONENT_SEPARATOR \
	COMPONENT_CODE(1, __VA_ARGS__) \

#define __DPV_FOREACH_COMPONENTS_3(COMPONENT_CODE, COMPONENT_SEPARATOR, ...) \
	COMPONENT_CODE(0, __VA_ARGS__) \
	COMPONENT_SEPARATOR \
	COMPONENT_CODE(1, __VA_ARGS__) \
	COMPONENT_SEPARATOR \
	COMPONENT_CODE(2, __VA_ARGS__) \

#define __DPV_FOREACH_COMPONENTS_4(COMPONENT_CODE, COMPONENT_SEPARATOR, ...) \
	COMPONENT_CODE(0, __VA_ARGS__) \
	COMPONENT_SEPARATOR \
	COMPONENT_CODE(1, __VA_ARGS__) \
	COMPONENT_SEPARATOR \
	COMPONENT_CODE(2, __VA_ARGS__) \
	COMPONENT_SEPARATOR \
	COMPONENT_CODE(3, __VA_ARGS__) \

#define __DPV_FOREACH_COMPONENTS(COMPONENTS, COMPONENT_CODE, COMPONENT_SEPARATOR, ...) \
	__DPV_FOREACH_COMPONENTS_##COMPONENTS(COMPONENT_CODE, COMPONENT_SEPARATOR, __VA_ARGS__)
	

// __DPV_FOREACH_COMPONENTS() helper to write per component code to be used as comma separated parameters to a function
#define __DPV_FOREACH_COMPONENTS_ARGS_2(COMPONENT_CODE, ...) \
	COMPONENT_CODE(0, __VA_ARGS__), \
	COMPONENT_CODE(1, __VA_ARGS__) \

#define __DPV_FOREACH_COMPONENTS_ARGS_3(COMPONENT_CODE, ...) \
	COMPONENT_CODE(0, __VA_ARGS__), \
	COMPONENT_CODE(1, __VA_ARGS__), \
	COMPONENT_CODE(2, __VA_ARGS__) \

#define __DPV_FOREACH_COMPONENTS_ARGS_4(COMPONENT_CODE, ...) \
	COMPONENT_CODE(0, __VA_ARGS__), \
	COMPONENT_CODE(1, __VA_ARGS__), \
	COMPONENT_CODE(2, __VA_ARGS__), \
	COMPONENT_CODE(3, __VA_ARGS__) \

#define __DPV_FOREACH_COMPONENTS_ARGS(COMPONENTS, COMPONENT_CODE, ...) \
	__DPV_FOREACH_COMPONENTS_ARGS_##COMPONENTS(COMPONENT_CODE, __VA_ARGS__)
	

// __DPV_FOREACH_COMPONENTS_CONSTRUCTOR() helper to write constructor for a dual pixel vector
#define __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, COMPONENT_CODE, ...) \
	__DPV_MULTI_VECTOR(TYPE, COMPONENTS)(__DPV_FOREACH_COMPONENTS_ARGS(COMPONENTS, COMPONENT_CODE, __VA_ARGS__))


// Helper to be used with __DPV_FOREACH_COMPONENTS() to call a function
#define __DPV_CODE_FUNCTION_2_ARGS(c, FUNCTION_NAME, ARG_NAME_0, ARG_NAME_1) FUNCTION_NAME(ARG_NAME_0[c], ARG_NAME_1[c])
#define __DPV_CODE_FUNCTION_3_ARGS(c, FUNCTION_NAME, ARG_NAME_0, ARG_NAME_1, ARG_NAME_2) FUNCTION_NAME(ARG_NAME_0[c], ARG_NAME_1[c], ARG_NAME_2[c])

// helper to be used with __DPV_FOREACH_COMPONENTS() to call an operator
#define __DPV_CODE_OPERATOR_VECTOR_VECTOR(c, ARG_NAME_0, OPERATOR, ARG_NAME_1) (ARG_NAME_0[c] OPERATOR ARG_NAME_1[c])
#define __DPV_CODE_OPERATOR_VECTOR_SCALAR(c, ARG_NAME_0, OPERATOR, ARG_NAME_1) (ARG_NAME_0[c] OPERATOR ARG_NAME_1)


// Helpers to define for each individual scalar and vector type
#define DEFINE_DPV_FUNCTION_FOR_FLOAT_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION(float) \

#define DEFINE_DPV_FUNCTION_FOR_INT_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION(uint) \
	DEFINE_DPV_FUNCTION(int) \
		
#define DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTOR(DEFINE_DPV_FUNCTION, COMPONENTS) \
	DEFINE_DPV_FUNCTION(float, COMPONENTS) \

#define DEFINE_DPV_FUNCTION_FOR_INT_VECTOR(DEFINE_DPV_FUNCTION, COMPONENTS) \
	DEFINE_DPV_FUNCTION(uint, COMPONENTS) \
	DEFINE_DPV_FUNCTION(int, COMPONENTS) \

#define DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTOR(DEFINE_DPV_FUNCTION, 2) \
	DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTOR(DEFINE_DPV_FUNCTION, 3) \
	DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTOR(DEFINE_DPV_FUNCTION, 4) \

#define DEFINE_DPV_FUNCTION_FOR_INT_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_INT_VECTOR(DEFINE_DPV_FUNCTION, 2) \
	DEFINE_DPV_FUNCTION_FOR_INT_VECTOR(DEFINE_DPV_FUNCTION, 3) \
	DEFINE_DPV_FUNCTION_FOR_INT_VECTOR(DEFINE_DPV_FUNCTION, 4) \
		
#if PLATFORM_SUPPORT_16BITS_DPV

	#define DEFINE_DPV_FUNCTION_FOR_HALF_SCALARS(DEFINE_DPV_FUNCTION) \
		DEFINE_DPV_FUNCTION(float16_t) \

	#define DEFINE_DPV_FUNCTION_FOR_SHORT_SCALARS(DEFINE_DPV_FUNCTION) \
		DEFINE_DPV_FUNCTION(uint16_t) \
		DEFINE_DPV_FUNCTION(int16_t) \
		
	#define DEFINE_DPV_FUNCTION_FOR_HALF_VECTORS(DEFINE_DPV_FUNCTION) \
		DEFINE_DPV_FUNCTION(float16_t, 2) \
		DEFINE_DPV_FUNCTION(float16_t, 3) \
		DEFINE_DPV_FUNCTION(float16_t, 4) \

	#define DEFINE_DPV_FUNCTION_FOR_HALF_VECTOR2(DEFINE_DPV_FUNCTION) \
		DEFINE_DPV_FUNCTION(float16_t, 2) \

	#define DEFINE_DPV_FUNCTION_FOR_SHORT_VECTORS(DEFINE_DPV_FUNCTION) \
		DEFINE_DPV_FUNCTION(uint16_t, 2) \
		DEFINE_DPV_FUNCTION(uint16_t, 3) \
		DEFINE_DPV_FUNCTION(uint16_t, 4) \
		DEFINE_DPV_FUNCTION(int16_t, 2) \
		DEFINE_DPV_FUNCTION(int16_t, 3) \
		DEFINE_DPV_FUNCTION(int16_t, 4) \
		
#else

	#define DEFINE_DPV_FUNCTION_FOR_HALF_SCALARS(DEFINE_DPV_FUNCTION)

	#define DEFINE_DPV_FUNCTION_FOR_SHORT_SCALARS(DEFINE_DPV_FUNCTION)
		
	#define DEFINE_DPV_FUNCTION_FOR_HALF_VECTORS(DEFINE_DPV_FUNCTION)

	#define DEFINE_DPV_FUNCTION_FOR_HALF_VECTOR2(DEFINE_DPV_FUNCTION)

	#define DEFINE_DPV_FUNCTION_FOR_SHORT_VECTORS(DEFINE_DPV_FUNCTION)
		
#endif

#define DEFINE_DPV_FUNCTION_FOR_ALL_32BIT_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_FLOAT_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_INT_SCALARS(DEFINE_DPV_FUNCTION) \

#define DEFINE_DPV_FUNCTION_FOR_ALL_16BIT_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_HALF_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_SHORT_SCALARS(DEFINE_DPV_FUNCTION) \

#define DEFINE_DPV_FUNCTION_FOR_ALL_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_ALL_32BIT_SCALARS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_ALL_16BIT_SCALARS(DEFINE_DPV_FUNCTION) \


#define DEFINE_DPV_FUNCTION_FOR_ALL_32BIT_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_INT_VECTORS(DEFINE_DPV_FUNCTION) \

#define DEFINE_DPV_FUNCTION_FOR_ALL_16BIT_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_HALF_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_SHORT_VECTORS(DEFINE_DPV_FUNCTION) \

#define DEFINE_DPV_FUNCTION_FOR_ALL_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_ALL_32BIT_VECTORS(DEFINE_DPV_FUNCTION) \
	DEFINE_DPV_FUNCTION_FOR_ALL_16BIT_VECTORS(DEFINE_DPV_FUNCTION) \



/* dpv_interleave_registers() & dpv_lo() & dpv_hi()
 *
 * Reinterleaves explicit vector floatN for each individual pixel into a floatNx2 matrix.
 * 
 * Example:
 *		half3 Color0 = ... ;
 *		half3 Color1 = ... ;
 *		
 *		half3x2 Color = dpv_interleave_registers(Color0, Color1);
 *		
 *		// Can access each individual channel with the [0] notation
 *		half2 Red = Color[0];
 *		half2 Green = Color[1];
 *		half2 Blue = Color[2];
 *		
 *		// And access back the per vectorized pixel data if need.
 *		Color0 = dpv_lo(Color);
 *		Color1 = dpv_hi(Color);
 */

#define DEFINE_DPV_FUNCTION_SCALAR(TYPE) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_interleave_registers(TYPE lo, TYPE hi) { return __DPV_MULTI_SCALAR(TYPE)(lo, hi); } \
	CALL_SITE_DEBUGLOC TYPE dpv_lo(__DPV_MULTI_SCALAR(TYPE) x) { return x[0]; } \
	CALL_SITE_DEBUGLOC TYPE dpv_hi(__DPV_MULTI_SCALAR(TYPE) x) { return x[1]; } \
	CALL_SITE_DEBUGLOC TYPE dpv_access_pixel(__DPV_MULTI_SCALAR(TYPE) x, const uint DualPixelId) { return x[DualPixelId]; } \

#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_interleave_registers(__DPV_VECTOR(TYPE, COMPONENTS) lo, __DPV_VECTOR(TYPE, COMPONENTS) hi) { return transpose(__DPV_MULTI_VECTOR_TRANSPOSED(TYPE, COMPONENTS)(lo, hi)); } \
	CALL_SITE_DEBUGLOC __DPV_VECTOR(TYPE, COMPONENTS) dpv_lo(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) x) { return transpose(x)[0]; } \
	CALL_SITE_DEBUGLOC __DPV_VECTOR(TYPE, COMPONENTS) dpv_hi(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) x) { return transpose(x)[1]; } \
	CALL_SITE_DEBUGLOC __DPV_VECTOR(TYPE, COMPONENTS) dpv_access_pixel(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) x, const uint DualPixelId) { return transpose(x)[DualPixelId]; } \

DEFINE_DPV_FUNCTION_FOR_ALL_SCALARS(DEFINE_DPV_FUNCTION_SCALAR)
DEFINE_DPV_FUNCTION_FOR_ALL_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_SCALAR
#undef DEFINE_DPV_FUNCTION_VECTOR


/* dpv_force_interleave_registers() to force using RDNA's v_pack_b32_b16
 *
 * Reinterleaves explicit vector floatN for each individual pixel into a floatNx2 matrix.
 * 
 * Example:
 *		half3x2 Color = dpv_force_interleave_registers(Color0, Color1);
 */

#define DEFINE_DPV_FUNCTION_SCALAR(TYPE) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_force_interleave_registers(TYPE lo, TYPE hi) { return dpv_interleave_registers(lo, hi); } \

#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_force_interleave_registers(__DPV_VECTOR(TYPE, COMPONENTS) lo, __DPV_VECTOR(TYPE, COMPONENTS) hi) { return dpv_interleave_registers(lo, hi); } \

DEFINE_DPV_FUNCTION_FOR_ALL_32BIT_SCALARS(DEFINE_DPV_FUNCTION_SCALAR)
DEFINE_DPV_FUNCTION_FOR_ALL_32BIT_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_SCALAR
#undef DEFINE_DPV_FUNCTION_VECTOR


#define DEFINE_DPV_FUNCTION_SCALAR(TYPE) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_force_interleave_registers(TYPE lo, TYPE hi) { return v_pack_b32_b16(lo, hi); } \

#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_force_interleave_registers(__DPV_VECTOR(TYPE, COMPONENTS) lo, __DPV_VECTOR(TYPE, COMPONENTS) hi) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_FUNCTION_2_ARGS, dpv_force_interleave_registers, lo, hi); \
	} \

DEFINE_DPV_FUNCTION_FOR_ALL_16BIT_SCALARS(DEFINE_DPV_FUNCTION_SCALAR)
DEFINE_DPV_FUNCTION_FOR_ALL_16BIT_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_SCALAR
#undef DEFINE_DPV_FUNCTION_VECTOR


/* dpv_interleave_array_registers() to interleave an array
 *
 * Reinterleaves explicit vector floatN for each individual pixel into a floatNx2 matrix.
 * 
 * Example:
 *		half3 ColorArray[2];
 *		ColorArray[0] = ... ;
 *		ColorArray[1] = ... ;
 *		
 *		half3x2 Color = dpv_interleave_registers_array(ColorArray);
 */
#define dpv_interleave_registers_array(ArrayName) dpv_interleave_registers((ArrayName)[0], (ArrayName)[1])
#define dpv_force_interleave_registers_array(ArrayName) dpv_force_interleave_registers((ArrayName)[0], (ArrayName)[1])


/* dpv_interleave_mono_registers()
 *
 * Reinterleaves explicit vector floatN for of one individual pixel into a floatNx2 matrix to reuse code for dual pixel vectorization.
 * 
 * Example:
 *		// transform color space a pair of colors
 *		half3x2 RGBToYCoCg(half3x2 Color) { ... }
 *		
 *		// transform color space of a color
 *		half3 RGBToYCoCg(half3 Color)
 *		{
 *			return dpv_lo(RGBToYCoCg(dpv_interleave_mono_registers(Color)));
 *		}
 */
#define DEFINE_DPV_FUNCTION_SCALAR(TYPE) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_interleave_mono_registers(TYPE lo) { return dpv_interleave_registers(lo, lo); }

#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_interleave_mono_registers(__DPV_VECTOR(TYPE, COMPONENTS) lo) { return dpv_interleave_registers(lo, lo); }

DEFINE_DPV_FUNCTION_FOR_ALL_SCALARS(DEFINE_DPV_FUNCTION_SCALAR)
DEFINE_DPV_FUNCTION_FOR_ALL_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_SCALAR
#undef DEFINE_DPV_FUNCTION_VECTOR


/* per vector multiplication
 *
 * Example:
 *		half3x2 Color = ...;
 *		half2 Coverage = ...;
 * 
 *		half3x2 BlueishColor = dpv_mul(Color, half3(1.00, 0.25, 0.25));
 *		half3x2 PremultipliedColor = dpv_scale(Color, Coverage);
 */
#define DEFINE_DPV_FUNCTION_SCALAR(TYPE) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_scale(__DPV_MULTI_SCALAR(TYPE) a, __DPV_MULTI_SCALAR(TYPE) b) { \
		return a * b; \
	} \

#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_mul(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_mul(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_mul(__DPV_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_scale(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_SCALAR(TYPE) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_SCALAR, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_scale(__DPV_MULTI_SCALAR(TYPE) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_SCALAR, b, *, a); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_scale(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, TYPE b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_SCALAR, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_scale(TYPE a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_SCALAR, b, *, a); \
	} \
	
DEFINE_DPV_FUNCTION_FOR_ALL_SCALARS(DEFINE_DPV_FUNCTION_SCALAR)
DEFINE_DPV_FUNCTION_FOR_ALL_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_SCALAR
#undef DEFINE_DPV_FUNCTION_VECTOR


/* per vector multiplication
 *
 * Example:
 *		half3x2 PosA = ...;
 *		half3 PosAToB = ...;
 * 
 *		half3x2 PosB = dpv_add(PosA, PosAToB);
 */
#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_add(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, +, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_add(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, +, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_add(__DPV_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, +, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_sub(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, -, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_sub(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, -, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_sub(__DPV_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, a, -, b); \
	} \

DEFINE_DPV_FUNCTION_FOR_ALL_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_VECTOR


/* per vector multiplication
 *
 * Example:
 *		half3x2 Color = ...;
 * 
 *		half2 Luma = dpv_dot(Color, half3(0.25, 0.5, 0.25));
 */
#define DEFINE_DPV_FUNCTION_SCALAR(TYPE) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_min(__DPV_MULTI_SCALAR(TYPE) a, __DPV_MULTI_SCALAR(TYPE) b) { \
		return min(a, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_max(__DPV_MULTI_SCALAR(TYPE) a, __DPV_MULTI_SCALAR(TYPE) b) { \
		return max(a, b); \
	} \

#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_dot(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS(COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, +, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_dot(__DPV_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS(COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, +, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_dot(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS(COMPONENTS, __DPV_CODE_OPERATOR_VECTOR_VECTOR, +, a, *, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_clamp(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) x, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) min, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) max) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_FUNCTION_3_ARGS, clamp, x, min, max); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_min(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return min(a, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_min(__DPV_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_FUNCTION_2_ARGS, min, a, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_min(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_FUNCTION_2_ARGS, min, a, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_max(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return max(a, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_max(__DPV_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_FUNCTION_2_ARGS, max, a, b); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_max(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_VECTOR(TYPE, COMPONENTS) b) { \
		return __DPV_FOREACH_COMPONENTS_CONSTRUCTOR(TYPE, COMPONENTS, __DPV_CODE_FUNCTION_2_ARGS, max, a, b); \
	} \
	
DEFINE_DPV_FUNCTION_FOR_ALL_SCALARS(DEFINE_DPV_FUNCTION_SCALAR)
DEFINE_DPV_FUNCTION_FOR_ALL_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_SCALAR
#undef DEFINE_DPV_FUNCTION_VECTOR


// 2D cross product.
#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_VECTOR(TYPE, COMPONENTS) dpv_cross(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) a, __DPV_MULTI_VECTOR(TYPE, COMPONENTS) b) { \
		__DPV_VECTOR(TYPE, COMPONENTS) lo = dpv_lo(b); \
		__DPV_VECTOR(TYPE, COMPONENTS) lo_rotated = __DPV_VECTOR(TYPE, COMPONENTS)(lo.y, -lo.x); \
		__DPV_VECTOR(TYPE, COMPONENTS) hi = dpv_hi(b); \
		__DPV_VECTOR(TYPE, COMPONENTS) hi_rotated = __DPV_VECTOR(TYPE, COMPONENTS)(hi.y, -hi.x); \
		__DPV_MULTI_VECTOR(TYPE, COMPONENTS) rotated = dpv_interleave_registers(lo_rotated, hi_rotated); \
		return dpv_dot(a, rotated); \
	}

DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTOR(DEFINE_DPV_FUNCTION_VECTOR, 2)
DEFINE_DPV_FUNCTION_FOR_HALF_VECTOR2(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_VECTOR


/* per vector vectorial operations
 *
 * Example:
 *		half3x2 WorldPositionDelta = ...;
 * 
 *		half2 Distance = dpv_length(WorldPositionDelta);
 */
#define DEFINE_DPV_FUNCTION_VECTOR(TYPE, COMPONENTS) \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_length2(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) v) { \
		return dpv_dot(v, v); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_SCALAR(TYPE) dpv_length(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) v) { \
		return sqrt(dpv_length2(v)); \
	} \
	CALL_SITE_DEBUGLOC __DPV_MULTI_VECTOR(TYPE, COMPONENTS) dpv_normalize(__DPV_MULTI_VECTOR(TYPE, COMPONENTS) v) { \
		return dpv_scale(v, rsqrt(dpv_length2(v))); \
	} \


DEFINE_DPV_FUNCTION_FOR_FLOAT_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
DEFINE_DPV_FUNCTION_FOR_HALF_VECTORS(DEFINE_DPV_FUNCTION_VECTOR)
#undef DEFINE_DPV_FUNCTION_VECTOR
