// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FP16Math.usf: Defines for using FP16 or FP32
=============================================================================*/
  
#pragma once

#ifndef EXPERIMENTAL_FP16
#define EXPERIMENTAL_FP16 1
#endif

#ifndef USE_MIN16FLOAT
#define USE_MIN16FLOAT 0
#endif

#if EXPERIMENTAL_FP16
#if USE_MIN16FLOAT
	#define EXP_FLT min16float
	#define EXP_FLT2 min16float2
	#define EXP_FLT3 min16float3
	#define EXP_FLT4 min16float4
	#define EXP_FLT3x3 min16float3x3
	#define EXP_FLT4x4 min16float4x4
	#define EXP_FLT4x3 min16float4x3
#else
	#define EXP_FLT half
	#define EXP_FLT2 half2
	#define EXP_FLT3 half3
	#define EXP_FLT4 half4
	#define EXP_FLT3x3 half3x3
	#define EXP_FLT4x4 half4x4
	#define EXP_FLT4x3 half4x3
#endif
#else
	#define EXP_FLT float
	#define EXP_FLT2 float2
	#define EXP_FLT3 float3
	#define EXP_FLT4 float4
	#define EXP_FLT3x3 float3x3
	#define EXP_FLT4x4 float4x4
	#define EXP_FLT4x3 float4x3
#endif
