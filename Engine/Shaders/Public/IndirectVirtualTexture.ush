// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/IndirectVirtualTextureDefinitions.h"

#ifdef __cplusplus
typedef Buffer<uint2> FIndirectVirtualTexture;
#else // __cplusplus
// HLSL Cross-Compiler Workaround
#define FIndirectVirtualTexture Buffer<uint2>
#endif // __cplusplus
