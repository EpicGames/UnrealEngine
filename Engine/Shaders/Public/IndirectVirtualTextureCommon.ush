// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/IndirectVirtualTexture.ush"
#include "/Engine/Private/VirtualTextureCommon.ush"

struct FIndirectVirtualTextureInfo
{
	uint BlockX;
	uint BlockY;
	uint BlockWidth;
	uint BlockHeight;
	uint MaxLevel;
};

FIndirectVirtualTextureUniform GetIndirectVirtualTextureUniform(uint4 MaterialTextureCollectionPackedUniforms0, uint4 MaterialTextureCollectionPackedUniforms1
																		  , uint4 MaterialTextureCollectionPackedUniforms2, uint4 MaterialTextureCollectionPackedUniforms3)
{
	FIndirectVirtualTextureUniform Out = (FIndirectVirtualTextureUniform)0;
#if TEXTURE_COLLECTION_PACKED_UNIFORMS
	Out.UniformCountSub1          = MaterialTextureCollectionPackedUniforms0.w;
	Out.PackedPageTableUniform[0] = MaterialTextureCollectionPackedUniforms1;
	Out.PackedPageTableUniform[1] = MaterialTextureCollectionPackedUniforms2;
	Out.PackedUniform             = MaterialTextureCollectionPackedUniforms3;
#endif // TEXTURE_COLLECTION_PACKED_UNIFORMS
	return Out;
}

uint GetIndirectVirtualTextureCount(in FIndirectVirtualTextureUniform Uniform)
{
	return Uniform.UniformCountSub1 + 1;
}

FIndirectVirtualTextureEntry LoadIndirectVirtualTexture(in FIndirectVirtualTexture Collection, in FIndirectVirtualTextureUniform Uniform, uint CollectionIndex)
{
	CollectionIndex = min(Uniform.UniformCountSub1, CollectionIndex);

	FIndirectVirtualTextureEntry Out = (FIndirectVirtualTextureEntry)0;
	Out.PackedCoordinateAndSize = Collection[CollectionIndex];
	return Out;
}

FIndirectVirtualTextureInfo UnpackIndirectVirtualTexture(in FIndirectVirtualTextureEntry Entry)
{
	FIndirectVirtualTextureInfo Out = (FIndirectVirtualTextureInfo)0;
	Out.BlockX      = Entry.PackedCoordinateAndSize.x  & 0xFFFF;
	Out.BlockY      = Entry.PackedCoordinateAndSize.x >> 16;
	Out.BlockWidth  = Entry.PackedCoordinateAndSize.y & 0xFFF;
	Out.BlockHeight = (Entry.PackedCoordinateAndSize.y >> 12) & 0xFFF;
	Out.MaxLevel    = Entry.PackedCoordinateAndSize.y >> 24;
	return Out;
}

VTPageTableUniform UnpackIndirectVirtualTexturePageTableUniform(in FIndirectVirtualTextureEntry Entry, uint4 A, uint4 B)
{
	VTPageTableUniform Out = VTPageTableUniform_Unpack(A, B);
	
	FIndirectVirtualTextureInfo Info = UnpackIndirectVirtualTexture(Entry);

	// Allow packing textures of different sizes and in turn number of mip levels
	// So, to avoid lower physical mip levels "leaking" in, limit the level
	// to that of this entry only.
	Out.MaxLevel = Info.MaxLevel;
	
	// This avoids both:
	//  1. Re-mapping the UV coordinates manually from 0-1 to the physical region
	//  2. Manual sampler addressing handling on the texture region borders
	Out.SizeInPages     = uint2(Info.BlockWidth, Info.BlockHeight);
	Out.XOffsetInPages += Info.BlockX;
	Out.YOffsetInPages += Info.BlockY;
	
	return Out;
}
