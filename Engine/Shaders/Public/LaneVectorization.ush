// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LaneVectorization.ush: Vectorize arbitrary number of processing per lane.
=============================================================================*/
  
#pragma once

#include "Platform.ush"
#include "WaveBroadcastIntrinsics.ush"


//------------------------------------------------------- DEFINE

#ifndef TENSOR_REGISTER_LAYOUT
	#define TENSOR_REGISTER_LAYOUT FRegisters_AoS_VectorArray
#endif

#ifndef TENSOR_REGISTER_TIGHTENING
	#define TENSOR_REGISTER_TIGHTENING 0
#endif


//------------------------------------------------------- GLOBAL VARIABLE

static uint GGroupThreadIndex = 0;


//------------------------------------------------------- STANDARD SCALAR OPERATIONS

bool not(bool x)
{
	return !x;
}

bool2 not(bool2 x)
{
	return !x;
}

bool3 not(bool3 x)
{
	return !x;
}

bool4 not(bool4 x)
{
	return !x;
}

uint bit_not(uint x)
{
	return ~x;
}

uint2 bit_not(uint2 x)
{
	return ~x;
}

#if PLATFORM_SUPPORTS_REAL_TYPES

uint16_t bit_not(uint16_t x)
{
	return ~x;
}

uint16_t2 bit_not(uint16_t2 x)
{
	return ~x;
}
#endif

bool2 v_pack_b32_b16(bool a, bool b)
{
	return bool2(a, b);
}

float2 v_pack_b32_b16(float a, float b)
{
	return float2(a, b);
}

uint2 v_pack_b32_b16(uint a, uint b)
{
	return uint2(a, b);
}

int2 v_pack_b32_b16(int a, int b)
{
	return int2(a, b);
}

float fast_sign(float x)
{
    return clamp(x * asfloat(0x7f7fffff), float(-1.0), float(1.0));
}

float2 fast_sign(float2 x)
{
    return clamp(x * asfloat(0x7f7fffff), float(-1.0), float(1.0));
}

#if PLATFORM_SUPPORTS_REAL_TYPES

half fast_sign(half x)
{
    return clamp(x * asfloat16(uint16_t(0x7bff)), half(-1.0), half(1.0));
}

half2 fast_sign(half2 x)
{
    return clamp(x * asfloat16(uint16_t(0x7bff)), half(-1.0), half(1.0));
}

#endif

uint bit_and(uint a, uint b)
{
    return a & b;
}

uint2 bit_and(uint2 a, uint2 b)
{
    return a & b;
}

uint bit_or(uint a, uint b)
{
    return a | b;
}

uint2 bit_or(uint2 a, uint2 b)
{
    return a | b;
}

#if PLATFORM_SUPPORTS_REAL_TYPES

uint16_t bit_and(uint16_t a, uint16_t b)
{
    return a & b;
}

uint16_t2 bit_and(uint16_t2 a, uint16_t2 b)
{
    return a & b;
}

uint16_t bit_or(uint16_t a, uint16_t b)
{
    return a | b;
}

uint16_t2 bit_or(uint16_t2 a, uint16_t2 b)
{
    return a | b;
}

#endif

uint bit_shift_left(uint a, uint b)
{
    return a << b;
}

uint2 bit_shift_left(uint2 a, uint2 b)
{
    return a << b;
}

uint bit_shift_right(uint a, uint b)
{
    return a >> b;
}

uint2 bit_shift_right(uint2 a, uint2 b)
{
    return a >> b;
}

#if PLATFORM_SUPPORTS_REAL_TYPES

uint16_t bit_shift_left(uint16_t a, uint16_t b)
{
    return a << b;
}

uint16_t2 bit_shift_left(uint16_t2 a, uint16_t2 b)
{
    return a << b;
}

uint16_t bit_shift_right(uint16_t a, uint16_t b)
{
    return a >> b;
}

uint16_t2 bit_shift_right(uint16_t2 a, uint16_t2 b)
{
    return a >> b;
}

#endif


//------------------------------------------------------- TIGHT REGISTERS

bool  PackRegistersTightly(bool v)
{
	return v;
}

bool2 PackRegistersTightly(bool2 v)
{
	return v;
}

bool3 PackRegistersTightly(bool3 v)
{
	return v;
}

bool4 PackRegistersTightly(bool4 v)
{
	return v;
}

float  PackRegistersTightly(float v)
{
	return v;
}

float2 PackRegistersTightly(float2 v)
{
	return v;
}

float3 PackRegistersTightly(float3 v)
{
	return v;
}

float4 PackRegistersTightly(float4 v)
{
	return v;
}

uint  PackRegistersTightly(uint v)
{
	return v;
}

uint2 PackRegistersTightly(uint2 v)
{
	return v;
}

uint3 PackRegistersTightly(uint3 v)
{
	return v;
}

uint4 PackRegistersTightly(uint4 v)
{
	return v;
}

int  PackRegistersTightly(int v)
{
	return v;
}

int2 PackRegistersTightly(int2 v)
{
	return v;
}

int3 PackRegistersTightly(int3 v)
{
	return v;
}

int4 PackRegistersTightly(int4 v)
{
	return v;
}


#if PLATFORM_SUPPORTS_REAL_TYPES

half  PackRegistersTightly(half v)
{
	return v;
}

half2 PackRegistersTightly(half2 v)
{
	return v_pack_b32_b16(v.x, v.y);
}

half3 PackRegistersTightly(half3 v)
{
	return half3(v_pack_b32_b16(v.x, v.y), v.z);
}

half4 PackRegistersTightly(half4 v)
{
	return half4(v_pack_b32_b16(v.x, v.y), v_pack_b32_b16(v.z, v.w));
}

uint16_t  PackRegistersTightly(uint16_t v)
{
	return v;
}

uint16_t2 PackRegistersTightly(uint16_t2 v)
{
	return v_pack_b32_b16(v.x, v.y);
}

uint16_t3 PackRegistersTightly(uint16_t3 v)
{
	return uint16_t3(v_pack_b32_b16(v.x, v.y), v.z);
}

uint16_t4 PackRegistersTightly(uint16_t4 v)
{
	return uint16_t4(v_pack_b32_b16(v.x, v.y), v_pack_b32_b16(v.z, v.w));
}

int16_t  PackRegistersTightly(int16_t v)
{
	return v;
}

int16_t2 PackRegistersTightly(int16_t2 v)
{
	return v_pack_b32_b16(v.x, v.y);
}

int16_t3 PackRegistersTightly(int16_t3 v)
{
	return int16_t3(v_pack_b32_b16(v.x, v.y), v.z);
}

int16_t4 PackRegistersTightly(int16_t4 v)
{
	return int16_t4(v_pack_b32_b16(v.x, v.y), v_pack_b32_b16(v.z, v.w));
}


#endif // PLATFORM_SUPPORTS_REAL_TYPES


//------------------------------------------------------- DWORD PACKING/UNPACKING

/** Returns the byte size of vector<ScalarType, VectorSize> */
#define GetVectorByteSize(ScalarType, VectorSize) uint(sizeof(ScalarType) * uint(VectorSize))

/** Returns the dowrd size of vector<ScalarType, VectorSize> */
#define GetVectorDwordSize(ScalarType, VectorSize) (uint(GetVectorByteSize(ScalarType, VectorSize) + 3u) / 4u)

// uint
void PackVectorToDwords(uint V, out uint DW[1])
{
	DW[0] = V;
}

void PackVectorToDwords(uint2 V, out uint DW[2])
{
	DW[0] = V.x;
	DW[1] = V.y;
}

void PackVectorToDwords(uint3 V, out uint DW[3])
{
	DW[0] = V.x;
	DW[1] = V.y;
	DW[2] = V.z;
}

void PackVectorToDwords(uint4 V, out uint DW[4])
{
	DW[0] = V.x;
	DW[1] = V.y;
	DW[2] = V.z;
	DW[3] = V.w;
}

void UnpackDwordsToVector(uint DW[1], out uint V)
{
	V   = DW[0];
}

void UnpackDwordsToVector(uint DW[2], out uint2 V)
{
	V.x = DW[0];
	V.y = DW[1];
}

void UnpackDwordsToVector(uint DW[3], out uint3 V)
{
	V.x = DW[0];
	V.y = DW[1];
	V.z = DW[2];
}

void UnpackDwordsToVector(uint DW[4], out uint4 V)
{
	V.x = DW[0];
	V.y = DW[1];
	V.z = DW[2];
	V.w = DW[3];
}

// float
void PackVectorToDwords(float V, out uint DW[1])
{
	DW[0] = asuint(V);
}

void PackVectorToDwords(float2 V, out uint DW[2])
{
	DW[0] = asuint(V.x);
	DW[1] = asuint(V.y);
}

void PackVectorToDwords(float3 V, out uint DW[3])
{
	DW[0] = asuint(V.x);
	DW[1] = asuint(V.y);
	DW[2] = asuint(V.z);
}

void PackVectorToDwords(float4 V, out uint DW[4])
{
	DW[0] = asuint(V.x);
	DW[1] = asuint(V.y);
	DW[2] = asuint(V.z);
	DW[3] = asuint(V.w);
}

void UnpackDwordsToVector(uint DW[1], out float V)
{
	V   = asfloat(DW[0]);
}

void UnpackDwordsToVector(uint DW[2], out float2 V)
{
	V.x = asfloat(DW[0]);
	V.y = asfloat(DW[1]);
}

void UnpackDwordsToVector(uint DW[3], out float3 V)
{
	V.x = asfloat(DW[0]);
	V.y = asfloat(DW[1]);
	V.z = asfloat(DW[2]);
}

void UnpackDwordsToVector(uint DW[4], out float4 V)
{
	V.x = asfloat(DW[0]);
	V.y = asfloat(DW[1]);
	V.z = asfloat(DW[2]);
	V.w = asfloat(DW[3]);
}

#if PLATFORM_SUPPORTS_REAL_TYPES

#if COMPILER_SUPPORT_UINT16_BITCAST

// uint16_t
void PackVectorToDwords(uint16_t V, out uint DW[1])
{
	DW[0] = uint(V);
}

void PackVectorToDwords(uint16_t2 V, out uint DW[1])
{
	DW[0] = bit_cast_uint(V.xy);
}

void PackVectorToDwords(uint16_t3 V, out uint DW[2])
{
	DW[0] = bit_cast_uint(V.xy);
	DW[1] = uint(V.z);
}

void PackVectorToDwords(uint16_t4 V, out uint DW[2])
{
	DW[0] = bit_cast_uint(V.xy);
	DW[1] = bit_cast_uint(V.zw);
}

void UnpackDwordsToVector(uint DW[1], out uint16_t V)
{
	V = uint16_t(DW[0]);
}

void UnpackDwordsToVector(uint DW[1], out uint16_t2 V)
{
	V.xy = bit_cast_uint16_t2(DW[0]);
}

void UnpackDwordsToVector(uint DW[2], out uint16_t3 V)
{
	V.xy = bit_cast_uint16_t2(DW[0]);
	V.z = uint16_t(DW[1]);
}

void UnpackDwordsToVector(uint DW[2], out uint16_t4 V)
{
	V.xy = bit_cast_uint16_t2(DW[0]);
	V.zw = bit_cast_uint16_t2(DW[1]);
}

#else // !COMPILER_SUPPORT_UINT16_BITCAST

// uint16_t
void PackVectorToDwords(uint16_t V, out uint DW[1])
{
	DW[0] = uint(V);
}

void PackVectorToDwords(uint16_t2 V, out uint DW[1])
{
	DW[0] = uint(V.x) | (uint(V.y) << 16u);
}

void PackVectorToDwords(uint16_t3 V, out uint DW[2])
{
	DW[0] = uint(V.x) | (uint(V.y) << 16u);
	DW[1] = uint(V.z);
}

void PackVectorToDwords(uint16_t4 V, out uint DW[2])
{
	DW[0] = uint(V.x) | (uint(V.y) << 16u);
	DW[1] = uint(V.z) | (uint(V.w) << 16u);
}

void UnpackDwordsToVector(uint DW[1], out uint16_t V)
{
	V = uint16_t(DW[0]);
}

void UnpackDwordsToVector(uint DW[1], out uint16_t2 V)
{
	V.x = uint16_t((DW[0] >>  0u) & 0xFFFFu);
	V.y = uint16_t((DW[0] >> 16u) & 0xFFFFu);
}

void UnpackDwordsToVector(uint DW[2], out uint16_t3 V)
{
	V.x = uint16_t((DW[0] >>  0u) & 0xFFFFu);
	V.y = uint16_t((DW[0] >> 16u) & 0xFFFFu);
	V.z = uint16_t((DW[1] >>  0u) & 0xFFFFu);
}

void UnpackDwordsToVector(uint DW[2], out uint16_t4 V)
{
	V.x = uint16_t((DW[0] >>  0u) & 0xFFFFu);
	V.y = uint16_t((DW[0] >> 16u) & 0xFFFFu);
	V.z = uint16_t((DW[1] >>  0u) & 0xFFFFu);
	V.w = uint16_t((DW[1] >> 16u) & 0xFFFFu);
}

#endif // !COMPILER_SUPPORT_UINT16_BITCAST

// half
void PackVectorToDwords(half V, out uint DW[1])
{
	PackVectorToDwords(asuint16(V), /* out */ DW);
}

void PackVectorToDwords(half2 V, out uint DW[1])
{
	PackVectorToDwords(uint16_t2(asuint16(V.x), asuint16(V.y)), /* out */ DW);
}

void PackVectorToDwords(half3 V, out uint DW[2])
{
	PackVectorToDwords(uint16_t3(asuint16(V.x), asuint16(V.y), asuint16(V.z)), /* out */ DW);
}

void PackVectorToDwords(half4 V, out uint DW[2])
{
	PackVectorToDwords(uint16_t4(asuint16(V.x), asuint16(V.y), asuint16(V.z), asuint16(V.w)), /* out */ DW);
}

void UnpackDwordsToVector(uint DW[1], out half V)
{
	uint16_t UV;
	UnpackDwordsToVector(DW, /* out */ UV);
	V = asfloat16(UV);
}

void UnpackDwordsToVector(uint DW[1], out half2 V)
{
	uint16_t2 UV;
	UnpackDwordsToVector(DW, /* out */ UV);
	V.x = asfloat16(UV.x);
	V.y = asfloat16(UV.y);
}

void UnpackDwordsToVector(uint DW[2], out half3 V)
{
	uint16_t3 UV;
	UnpackDwordsToVector(DW, /* out */ UV);
	V.x = asfloat16(UV.x);
	V.y = asfloat16(UV.y);
	V.z = asfloat16(UV.z);
}

void UnpackDwordsToVector(uint DW[2], out half4 V)
{
	uint16_t4 UV;
	UnpackDwordsToVector(DW, /* out */ UV);
	V.x = asfloat16(UV.x);
	V.y = asfloat16(UV.y);
	V.z = asfloat16(UV.z);
	V.w = asfloat16(UV.w);
}


#endif // PLATFORM_SUPPORTS_REAL_TYPES

//------------------------------------------------------- UTIL FUNCTION FOR MANUAL LOOP UNROLL

uint GetDW(uint DW[1], const uint Index)
{
	return DW[Index];
}

uint GetDW(uint DW[2], const uint Index)
{
	return DW[Index];
}

uint GetDW(uint DW[3], const uint Index)
{
	return DW[Index];
}

uint GetDW(uint DW[4], const uint Index)
{
	return DW[Index];
}

void SetDW(inout uint DW[1], const uint Index, uint V)
{
	DW[Index] = V;
}

void SetDW(inout uint DW[2], const uint Index, uint V)
{
	DW[Index] = V;
}

void SetDW(inout uint DW[3], const uint Index, uint V)
{
	DW[Index] = V;
}

void SetDW(inout uint DW[4], const uint Index, uint V)
{
	DW[Index] = V;
}

//------------------------------------------------------- GROUP SHARED COMMUNICATION

#if defined(LDS_SIZE)

groupshared uint SharedData[LDS_SIZE * LDS_DWORD_COMPONENT_COUNT];

void WriteDwordToLDS(const uint SharedIndex, uint V)
{
	SharedData[SharedIndex] = V;
}

uint ReadDwordFromLDS(const uint SharedIndex)
{
	return SharedData[SharedIndex];
}

void AtomicIncrementLDSDword(const uint SharedIndex, uint V)
{
	InterlockedAdd(/* inout */ SharedData[SharedIndex], V);
}

template<typename ScalarType, uint VectorSize>
void WriteVectorToLDS(const uint SharedIndex, vector<ScalarType, VectorSize> V)
{
	// manual loop unroll the most used cases to reduce compile time
	// the branch will be compiled away
	uint DW[GetVectorDwordSize(ScalarType, VectorSize)];
	PackVectorToDwords(V, /* out */ DW);

	if (GetVectorDwordSize(ScalarType, VectorSize) == 1)
	{
		SharedData[SharedIndex + 0 * LDS_SIZE] = GetDW(DW, 0);
	}
	else if (GetVectorDwordSize(ScalarType, VectorSize) == 2)
	{
		SharedData[SharedIndex + 0 * LDS_SIZE] = GetDW(DW, 0);
		SharedData[SharedIndex + 1 * LDS_SIZE] = GetDW(DW, 1);
	}
	else
	{
		UNROLL
		for (uint i = 0; i < GetVectorDwordSize(ScalarType, VectorSize); i++)
		{
			SharedData[SharedIndex + i * LDS_SIZE] = GetDW(DW, i);
		}
	}
}

template<typename ScalarType, uint VectorSize>
void ReadVectorFromLDS(const uint SharedIndex, out vector<ScalarType, VectorSize> V)
{
	uint DW[GetVectorDwordSize(ScalarType, VectorSize)];

	if (GetVectorDwordSize(ScalarType, VectorSize) == 1)
	{
		SetDW(DW, 0, SharedData[SharedIndex + 0 * LDS_SIZE]);
	}
	else if (GetVectorDwordSize(ScalarType, VectorSize) == 2)
	{
		SetDW(DW, 0, SharedData[SharedIndex + 0 * LDS_SIZE]);
		SetDW(DW, 1, SharedData[SharedIndex + 1 * LDS_SIZE]);
	}
	else
	{
		UNROLL
		for (uint i = 0; i < GetVectorDwordSize(ScalarType, VectorSize); i++)
		{
			SetDW(DW, i, SharedData[SharedIndex + i * LDS_SIZE]);
		}
	}
	UnpackDwordsToVector(DW, /* out */ V);
}

#endif // defined(LDS_SIZE)

//------------------------------------------------------- MANUAL LDS DESPILL

#if defined(LDS_DESPILL_DWORD_COUNT)

groupshared uint DespilledData[LDS_DESPILL_DWORD_COUNT];

#endif // defined(LDS_DESPILL_DWORD_COUNT)


//------------------------------------------------------- REGISTERS LAYOUTS

/** Stores one unique vector<>. InElementCount must be == 1  */
template<typename ScalarType, uint InVectorSize, uint InElementCount>
struct FRegisters_S_OneVector
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint ElementCount = 1;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize = VectorSize;
	static const uint RegisterRowCount = 1;

	vector<ScalarType, VectorSize> E;
	

	// ------------- register rows

	CALL_SITE_DEBUGLOC
	vector<ScalarType, RegisterRowSize> GetRegisterRow(const uint RegisterRowIndex)
	{
		return E;
	}
	
	CALL_SITE_DEBUGLOC
	void SetRegisterRow(const uint RegisterRowIndex, vector<ScalarType, RegisterRowSize> RegisterRow)
	{
		E = RegisterRow;
	}
	
	
	// ------------- elements

	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		return E;
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		E = Element;
	}

	vector<ScalarType, VectorSize> WaveBroadcastElement(const FWaveBroadcastSettings BroadcastSettings, const uint ElementIndex)
	{
		return WaveBroadcast(BroadcastSettings, GetElement(ElementIndex));
	}
	
	
	// ------------- elements' components

	CALL_SITE_DEBUGLOC
	ScalarType GetElementComponent(const uint ElementIndex, const uint ComponentIndex)
	{
		return E[ComponentIndex];
	}

	CALL_SITE_DEBUGLOC
	void SetElementComponent(const uint ElementIndex, const uint ComponentIndex, ScalarType Component)
	{
		E[ComponentIndex] = Component;
	}
}; // struct FRegisters_S_OneVector

/** Stores an array of vector<> as an array of structure in registers.  */
template<typename ScalarType, uint InVectorSize, uint InElementCount>
struct FRegisters_AoS_VectorArray
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint ElementCount = InElementCount;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize = VectorSize;
	static const uint RegisterRowCount = ElementCount;

	vector<ScalarType, VectorSize> Array[ElementCount];
	
	
	// ------------- register rows
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, RegisterRowSize> GetRegisterRow(const uint RegisterRowIndex)
	{
		return Array[RegisterRowIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetRegisterRow(const uint RegisterRowIndex, vector<ScalarType, RegisterRowSize> RegisterRow)
	{
		Array[RegisterRowIndex] = RegisterRow;
	}
	

	// ------------- elements
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		return Array[ElementIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		Array[ElementIndex] = Element;
	}

	vector<ScalarType, VectorSize> WaveBroadcastElement(const FWaveBroadcastSettings BroadcastSettings, const uint ElementIndex)
	{
		return WaveBroadcast(BroadcastSettings, GetElement(ElementIndex));
	}
	

	// ------------- elements' components

	CALL_SITE_DEBUGLOC
	ScalarType GetElementComponent(const uint ElementIndex, const uint ComponentIndex)
	{
		return Array[ElementIndex][ComponentIndex];
	}

	CALL_SITE_DEBUGLOC
	void SetElementComponent(const uint ElementIndex, const uint ComponentIndex, ScalarType Component)
	{
		Array[ElementIndex][ComponentIndex] = Component;
	}
}; // struct FRegisters_AoS_VectorArray

/** Stores an array of vector<> as an array of structure in registers using a matrix<>. */
template<typename ScalarType, uint InVectorSize, uint InElementCount>
struct FRegisters_AoS_Matrix
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint ElementCount = InElementCount;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize = VectorSize;
	static const uint RegisterRowCount = ElementCount;

	matrix<ScalarType, ElementCount, VectorSize> M;
	
	
	// ------------- register rows
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, RegisterRowSize> GetRegisterRow(const uint RegisterRowIndex)
	{
		return M[RegisterRowIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetRegisterRow(const uint RegisterRowIndex, vector<ScalarType, RegisterRowSize> RegisterRow)
	{
		M[RegisterRowIndex] = RegisterRow;
	}
	

	// ------------- elements
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		return M[ElementIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		M[ElementIndex] = Element;
	}

	vector<ScalarType, VectorSize> WaveBroadcastElement(const FWaveBroadcastSettings BroadcastSettings, const uint ElementIndex)
	{
		return WaveBroadcast(BroadcastSettings, GetElement(ElementIndex));
	}
	
	
	// ------------- elements' components

	CALL_SITE_DEBUGLOC
	ScalarType GetElementComponent(const uint ElementIndex, const uint ComponentIndex)
	{
		return M[ElementIndex][ComponentIndex];
	}

	CALL_SITE_DEBUGLOC
	void SetElementComponent(const uint ElementIndex, const uint ComponentIndex, ScalarType Component)
	{
		M[ElementIndex][ComponentIndex] = Component;
	}
}; // struct FRegisters_AoS_Matrix

/** Stores an array of vector<> as a structure of array in registers using a matrix<>. Requires InElementCount a pair >= 2 */
template<typename ScalarType, uint InVectorSize, uint InElementCount>
struct FRegisters_SoA_Matrix
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint ElementCount = InElementCount;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize = ElementCount;
	static const uint RegisterRowCount = VectorSize;

	matrix<ScalarType, VectorSize, ElementCount> M;
	
	
	// ------------- register rows
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, RegisterRowSize> GetRegisterRow(const uint RegisterRowIndex)
	{
		return M[RegisterRowIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetRegisterRow(const uint RegisterRowIndex, vector<ScalarType, RegisterRowSize> RegisterRow)
	{
		M[RegisterRowIndex] = RegisterRow;
	}
	

	// ------------- elements
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		vector<ScalarType, VectorSize> Element;
		if (VectorSize == 1)
		{
			Element[0] = M[0][ElementIndex];
		}
		else if (VectorSize == 3)
		{
			Element[0] = M[0][ElementIndex];
			Element[1] = M[1][ElementIndex];
			Element[2] = M[2][ElementIndex];
		}
		else
		{
			UNROLL
			for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
			{
				Element[ComponentIndex] = M[ComponentIndex][ElementIndex];
			}
		}
		return Element;
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		if (VectorSize == 1)
		{
			M[0][ElementIndex] = Element[0];
		}
		else if (VectorSize == 3)
		{
			M[0][ElementIndex] = Element[0];
			M[1][ElementIndex] = Element[1];
			M[2][ElementIndex] = Element[2];
		}
		else
		{
			UNROLL
			for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
			{
				M[ComponentIndex][ElementIndex] = Element[ComponentIndex];
			}
		}
	}

	vector<ScalarType, VectorSize> WaveBroadcastElement(const FWaveBroadcastSettings BroadcastSettings, const uint ElementIndex)
	{
		vector<ScalarType, VectorSize> Element;
		
		if (VectorSize == 1)
		{
			Element[0] = WaveBroadcast(BroadcastSettings, M[0])[ElementIndex];
		}
		else if (VectorSize == 3)
		{
			Element[0] = WaveBroadcast(BroadcastSettings, M[0])[ElementIndex];
			Element[1] = WaveBroadcast(BroadcastSettings, M[1])[ElementIndex];
			Element[2] = WaveBroadcast(BroadcastSettings, M[2])[ElementIndex];
		}
		else
		{
			UNROLL
			for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
			{
				Element[ComponentIndex] = WaveBroadcast(BroadcastSettings, M[ComponentIndex])[ElementIndex];
			}
		}
		return Element;
	}
	
	
	// ------------- elements' components

	CALL_SITE_DEBUGLOC
	ScalarType GetElementComponent(const uint ElementIndex, const uint ComponentIndex)
	{
		return M[ComponentIndex][ElementIndex];
	}

	CALL_SITE_DEBUGLOC
	void SetElementComponent(const uint ElementIndex, const uint ComponentIndex, ScalarType Component)
	{
		M[ComponentIndex][ElementIndex] = Component;
	}
}; // struct FRegisters_SoA_Matrix

/** Stores an array of vector<> as a structure of array in registers using a array of vector<X, 2>. Requires InElementCount a pair >= 2 */
template<typename ScalarType, uint InVectorSize, uint InElementCount>
struct FRegisters_AoS_PairArray
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint ElementCount = InElementCount;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize = 2;
	static const uint RegisterRowCount = (ElementCount * VectorSize) / 2;

	vector<ScalarType, RegisterRowSize> M[RegisterRowCount];
	
	
	// ------------- register rows
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, RegisterRowSize> GetRegisterRow(const uint RegisterRowIndex)
	{
		return M[RegisterRowIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetRegisterRow(const uint RegisterRowIndex, vector<ScalarType, RegisterRowSize> RegisterRow)
	{
		M[RegisterRowIndex] = RegisterRow;
	}
	

	// ------------- elements
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		vector<ScalarType, VectorSize> Element;
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
		{
			Element[ComponentIndex] = GetElementComponent(ElementIndex, ComponentIndex);
		}
		return Element;
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
		{
			SetElementComponent(ElementIndex, ComponentIndex, Element[ComponentIndex]);
		}
	}

	vector<ScalarType, VectorSize> WaveBroadcastElement(const FWaveBroadcastSettings BroadcastSettings, const uint ElementIndex)
	{
		vector<ScalarType, VectorSize> Element;
		if (sizeof(ScalarType) == 4)
		{
			UNROLL
			for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
			{
				const uint ArrayIndex = ComponentIndex + VectorSize * ElementIndex;
				Element[ComponentIndex] = WaveBroadcast(BroadcastSettings, M[ArrayIndex / 2u][ArrayIndex % 2u]);
			}
		}
		else
		{
			const uint FirstArrayIndex = VectorSize * ElementIndex;
			
			vector<ScalarType, RegisterRowSize> BroadcastedArray[(VectorSize + 1u) / 2u];
			UNROLL
			for (uint BroadcastIndex = 0; BroadcastIndex < ((VectorSize + 1u) / 2u); BroadcastIndex++)
			{
				BroadcastedArray[BroadcastIndex] = WaveBroadcast(BroadcastSettings, M[FirstArrayIndex / 2u + BroadcastIndex]);
			}
			
			UNROLL
			for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
			{
				const uint BroadcastedArrayIndex = ComponentIndex + VectorSize * ElementIndex - (FirstArrayIndex / 2u) * 2u;
				Element[ComponentIndex] = BroadcastedArray[BroadcastedArrayIndex / 2u][BroadcastedArrayIndex % 2u];
			}
		}
		return Element;
	}
	
	
	// ------------- elements' components

	CALL_SITE_DEBUGLOC
	ScalarType GetElementComponent(const uint ElementIndex, const uint ComponentIndex)
	{
		const uint ArrayIndex = ComponentIndex + VectorSize * ElementIndex;
		return M[ArrayIndex / 2u][ArrayIndex % 2u];
	}

	CALL_SITE_DEBUGLOC
	void SetElementComponent(const uint ElementIndex, const uint ComponentIndex, ScalarType Component)
	{
		const uint ArrayIndex = ComponentIndex + VectorSize * ElementIndex;
		M[ArrayIndex / 2u][ArrayIndex % 2u] = Component;
	}
}; // struct FRegisters_AoS_PairArray

/** Stores an array of vector<> as a structure of array in registers using a array of vector<X, 2>. Requires InElementCount a pair >= 2 */
template<typename ScalarType, uint InVectorSize, uint InElementCount>
struct FRegisters_SoA_PairArray
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint ElementCount = InElementCount;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize = 2;
	static const uint RegisterRowCount = (ElementCount * VectorSize) / 2;

	vector<ScalarType, RegisterRowSize> M[RegisterRowCount];
	
	
	// ------------- register rows
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, RegisterRowSize> GetRegisterRow(const uint RegisterRowIndex)
	{
		return M[RegisterRowIndex];
	}
	
	CALL_SITE_DEBUGLOC
	void SetRegisterRow(const uint RegisterRowIndex, vector<ScalarType, RegisterRowSize> RegisterRow)
	{
		M[RegisterRowIndex] = RegisterRow;
	}
	

	// ------------- elements
	
	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		vector<ScalarType, VectorSize> Element;
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
		{
			Element[ComponentIndex] = GetElementComponent(ElementIndex, ComponentIndex);
		}
		return Element;
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
		{
			SetElementComponent(ElementIndex, ComponentIndex, Element[ComponentIndex]);
		}
	}

	vector<ScalarType, VectorSize> WaveBroadcastElement(const FWaveBroadcastSettings BroadcastSettings, const uint ElementIndex)
	{
		vector<ScalarType, VectorSize> Element;
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSize; ComponentIndex++)
		{
			const uint ArrayIndex = ElementIndex + VectorSize * ComponentIndex;
			Element[ComponentIndex] = WaveBroadcast(BroadcastSettings, M[ArrayIndex / 2u])[ArrayIndex % 2u];
		}
		return Element;
	}
	
	
	// ------------- elements' components

	CALL_SITE_DEBUGLOC
	ScalarType GetElementComponent(const uint ElementIndex, const uint ComponentIndex)
	{
		const uint ArrayIndex = ElementIndex + VectorSize * ComponentIndex;
		return M[ArrayIndex / 2u][ArrayIndex % 2u];
	}

	CALL_SITE_DEBUGLOC
	void SetElementComponent(const uint ElementIndex, const uint ComponentIndex, ScalarType Component)
	{
		const uint ArrayIndex = ElementIndex + VectorSize * ComponentIndex;
		M[ArrayIndex / 2u][ArrayIndex % 2u] = Component;
	}
}; // struct FRegisters_SoA_PairArray


//------------------------------------------------------- TENSORS

/** Packs a vector<> in 2 dimension. */
template<typename ScalarType, uint InVectorSize, uint InSimdSizeX, uint InSimdSizeY>
struct TLaneVector2D
{
	/** Size of each element. */
	static const uint VectorSize = InVectorSize;
	
	/** Number of elememnts */
	static const uint SimdSize = InSimdSizeX * InSimdSizeY;
	static const uint SimdSizeX = InSimdSizeX;
	static const uint SimdSizeY = InSimdSizeY;
	
	/** Size and number of register rows */
	static const uint RegisterRowSize  = TENSOR_REGISTER_LAYOUT<ScalarType, InVectorSize, InSimdSizeX * InSimdSizeY>::RegisterRowSize;
	static const uint RegisterRowCount = TENSOR_REGISTER_LAYOUT<ScalarType, InVectorSize, InSimdSizeX * InSimdSizeY>::RegisterRowCount;

	TENSOR_REGISTER_LAYOUT<ScalarType, InVectorSize, InSimdSizeX * InSimdSizeY> Registers;

	
	// ------------- getting and setting elements and components

	// Access and set individual element of the vector.
	CALL_SITE_DEBUGLOC
	vector<ScalarType, VectorSize> GetElement(const uint ElementIndex)
	{
		return Registers.GetElement(ElementIndex);
	}
	
	CALL_SITE_DEBUGLOC
	void SetElement(const uint ElementIndex, vector<ScalarType, VectorSize> Element)
	{
		Registers.SetElement(ElementIndex, Element);
	}
	
	CALL_SITE_DEBUGLOC
	void SetAllElements(vector<ScalarType, VectorSize> Element)
	{
		if(SimdSize == 2)
		{
			Registers.SetElement(0, Element);
			Registers.SetElement(1, Element);
		}
		else if (SimdSize == 4)
		{
			Registers.SetElement(0, Element);
			Registers.SetElement(1, Element);
			Registers.SetElement(2, Element);
			Registers.SetElement(3, Element);
		}
		else
		{
			UNROLL
			for (uint ElementIndex = 0; ElementIndex < SimdSize; ElementIndex++)
			{
				Registers.SetElement(ElementIndex, Element);
			}
		}
	}
	
	
	//  Access and set a component of the vector
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, 1, SimdSizeX, SimdSizeY> GetComponent(const uint ComponentIndex)
	{
		TLaneVector2D<ScalarType, 1, SimdSizeX, SimdSizeY> R;
		if(SimdSize == 2)
		{
			R.SetElement(0, Registers.GetElementComponent(0, ComponentIndex));
			R.SetElement(1, Registers.GetElementComponent(1, ComponentIndex));
		}
		else if (SimdSize == 4)
		{
			R.SetElement(0, Registers.GetElementComponent(0, ComponentIndex));
			R.SetElement(1, Registers.GetElementComponent(1, ComponentIndex));
			R.SetElement(2, Registers.GetElementComponent(2, ComponentIndex));
			R.SetElement(3, Registers.GetElementComponent(3, ComponentIndex));
		}
		else
		{
			UNROLL
			for (uint ElementIndex = 0; ElementIndex < SimdSize; ElementIndex++)
			{
				R.SetElement(ElementIndex, Registers.GetElementComponent(ElementIndex, ComponentIndex));
			}
		}
		return R;
	}
	
	CALL_SITE_DEBUGLOC
	void SetComponent(uint ComponentIndex, TLaneVector2D<ScalarType, 1, SimdSizeX, SimdSizeY> Comp)
	{
		if(SimdSize == 2)
		{
			Registers.SetElementComponent(0, ComponentIndex, Comp.GetElement(0));
			Registers.SetElementComponent(1, ComponentIndex, Comp.GetElement(1));
		}
		else if (SimdSize == 4)
		{
			Registers.SetElementComponent(0, ComponentIndex, Comp.GetElement(0));
			Registers.SetElementComponent(1, ComponentIndex, Comp.GetElement(1));
			Registers.SetElementComponent(2, ComponentIndex, Comp.GetElement(2));
			Registers.SetElementComponent(3, ComponentIndex, Comp.GetElement(3));
		}
		else
		{
			UNROLL
			for (uint ElementIndex = 0; ElementIndex < SimdSize; ElementIndex++)
			{
				Registers.SetElementComponent(ElementIndex, ComponentIndex, Comp.GetElement(ElementIndex));
			}
		}
	}
	
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, 1, SimdSizeX, SimdSizeY> operator [](const uint ComponentIndex)
	{
		return GetComponent(ComponentIndex);
	}
	

	// ------------- constructors

	/** Casts a scalar into a vector<>. */
	CALL_SITE_DEBUGLOC
	static TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> Vectorize(
		TLaneVector2D<ScalarType, 1, SimdSizeX, SimdSizeY> A)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(SimdSize == 2)
		{
			R.SetElement(0, A.GetElement(0));
			R.SetElement(1, A.GetElement(1));
		}
		else if (SimdSize == 4)
		{
			R.SetElement(0, A.GetElement(0));
			R.SetElement(1, A.GetElement(1));
			R.SetElement(2, A.GetElement(2));
			R.SetElement(3, A.GetElement(3));
		}
		else
		{
			UNROLL
			for (uint ElementIndex = 0; ElementIndex < SimdSize; ElementIndex++)
			{
				R.SetElement(ElementIndex, A.GetElement(ElementIndex));
			}
		}
		return R;
	}
	
	/** Initialize all elements with a single same vector<>. */
	CALL_SITE_DEBUGLOC
	static TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> Const(
		vector<ScalarType, VectorSize> A)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(SimdSize == 2)
		{
			R.SetElement(0, A);
			R.SetElement(1, A);
		}
		else if (SimdSize == 4)
		{
			R.SetElement(0, A);
			R.SetElement(1, A);
			R.SetElement(2, A);
			R.SetElement(3, A);
		}
		else
		{
			UNROLL
			for (uint ElementIndex = 0; ElementIndex < SimdSize; ElementIndex++)
			{
				R.SetElement(ElementIndex, A);
			}
		}
		return R;
	}
	
	/** Casts vector<A> to vector<B>. */
	CALL_SITE_DEBUGLOC
	template<typename SourceScalarType>
	static TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> CastFrom(
		TLaneVector2D<SourceScalarType, VectorSize, SimdSizeX, SimdSizeY> A)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(SimdSize == 2)
		{
			R.SetElement(0, vector<ScalarType, VectorSize>(A.GetElement(0)));
			R.SetElement(1, vector<ScalarType, VectorSize>(A.GetElement(1)));
		}
		else if (SimdSize == 4)
		{
			R.SetElement(0, vector<ScalarType, VectorSize>(A.GetElement(0)));
			R.SetElement(1, vector<ScalarType, VectorSize>(A.GetElement(1)));
			R.SetElement(2, vector<ScalarType, VectorSize>(A.GetElement(2)));
			R.SetElement(3, vector<ScalarType, VectorSize>(A.GetElement(3)));
		}
		else
		{
			UNROLL
			for (uint ElementIndex = 0; ElementIndex < SimdSize; ElementIndex++)
			{
				R.SetElement(ElementIndex, vector<ScalarType, VectorSize>(A.GetElement(ElementIndex)));
			}
		}
		return R;
	}
	
	/** Force tight register packing. */
	CALL_SITE_DEBUGLOC
	void TightenRegisters()
	#if TENSOR_REGISTER_TIGHTENING
	{
		if(RegisterRowCount == 2)
		{
			Registers.SetRegisterRow(0, PackRegistersTightly(Registers.GetRegisterRow(0)));
			Registers.SetRegisterRow(1, PackRegistersTightly(Registers.GetRegisterRow(1)));
		}
		else if (RegisterRowCount == 4)
		{
			Registers.SetRegisterRow(0, PackRegistersTightly(Registers.GetRegisterRow(0)));
			Registers.SetRegisterRow(1, PackRegistersTightly(Registers.GetRegisterRow(1)));
			Registers.SetRegisterRow(2, PackRegistersTightly(Registers.GetRegisterRow(2)));
			Registers.SetRegisterRow(3, PackRegistersTightly(Registers.GetRegisterRow(3)));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				Registers.SetRegisterRow(RegisterRowIndex, PackRegistersTightly(Registers.GetRegisterRow(RegisterRowIndex)));
			}
		}
	}
	#else
	{
		// NOP
	}
	#endif
	
	#if defined(LDS_DESPILL_DWORD_COUNT)
	
	/** Returns the LDS spill size in dwords */
	static uint GetLDSSpillDwordSize()
	{
		return RegisterRowCount * GetVectorDwordSize(ScalarType, RegisterRowSize);
	}
	
	/** Store to LDS spill to save VGPR. */
	CALL_SITE_DEBUGLOC
	void StoreLDSSpill(const uint DespillDwordOffset)
	{
		UNROLL
		for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
		{
			vector<ScalarType, RegisterRowSize> ResgiterRow = Registers.GetRegisterRow(RegisterRowIndex);
			
			uint ResgiterRowDW[GetVectorDwordSize(ScalarType, RegisterRowSize)];
			PackVectorToDwords(ResgiterRow, /* out */ ResgiterRowDW);
	
			UNROLL
			for (uint i = 0; i < GetVectorDwordSize(ScalarType, RegisterRowSize); i++)
			{
				DespilledData[GGroupThreadIndex + DespillDwordOffset + (RegisterRowIndex * GetVectorDwordSize(ScalarType, RegisterRowSize) + i) * LDS_DESPILL_THREAD_COUNT] = ResgiterRowDW[i];
			}
		}
	}
	
	/** Load from LDS spill to save VGPR. */
	CALL_SITE_DEBUGLOC
	static TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> LoadLDSSpill(const uint DespillDwordOffset)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> Vector;
		UNROLL
		for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
		{
			uint ResgiterRowDW[GetVectorDwordSize(ScalarType, RegisterRowSize)];
			UNROLL
			for (uint i = 0; i < GetVectorDwordSize(ScalarType, RegisterRowSize); i++)
			{
				ResgiterRowDW[i] = DespilledData[GGroupThreadIndex + DespillDwordOffset + (RegisterRowIndex * GetVectorDwordSize(ScalarType, RegisterRowSize) + i) * LDS_DESPILL_THREAD_COUNT];
			}
			
			vector<ScalarType, RegisterRowSize> ResgiterRow;
			UnpackDwordsToVector(ResgiterRowDW, /* out */ ResgiterRow);
	
			Vector.Registers.SetRegisterRow(RegisterRowIndex, ResgiterRow);
		}
		return Vector;
	}
	
	#endif // defined(LDS_DESPILL_DWORD_COUNT)
	
	
	// ------------- binary operator +
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator + (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) + B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) + B.Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) + B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) + B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) + B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) + B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) + B.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}
	
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator + (vector<ScalarType, VectorSize> B)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> BV;
		BV.SetAllElements(B);

		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) + BV.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) + BV.Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) + BV.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) + BV.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) + BV.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) + BV.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) + BV.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}

	
	// ------------- unary operator -
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator - ()
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, -Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, -Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, -Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, -Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, -Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, -Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, -Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}
	
	// ------------- binary operator -
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator - (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) - B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) - B.Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) - B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) - B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) - B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) - B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) - B.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}
	
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator - (vector<ScalarType, VectorSize> B)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> BV;
		BV.SetAllElements(B);

		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) - BV.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) - BV.Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) - BV.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) - BV.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) - BV.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) - BV.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) - BV.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}

	
	// ------------- binary operator *
	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator * (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) * B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) * B.Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) * B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) * B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) * B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) * B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) * B.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}

	CALL_SITE_DEBUGLOC
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> operator * (vector<ScalarType, VectorSize> B)
	{
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> BV;
		BV.SetAllElements(B);

		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) * BV.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) * BV.Registers.GetRegisterRow(1));
		}
		else if (RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) * BV.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) * BV.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) * BV.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) * BV.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) * BV.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}
	
	// ------------- comparison operators
	CALL_SITE_DEBUGLOC
	TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> operator > (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> R;
		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) > B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) > B.Registers.GetRegisterRow(1));
		}
		else if(RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) > B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) > B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) > B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) > B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) > B.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}
	
	CALL_SITE_DEBUGLOC
	TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> operator < (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> R;

		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) < B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) < B.Registers.GetRegisterRow(1));
		}
		else if(RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) < B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) < B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) < B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) < B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
			for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
			{
				R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) < B.Registers.GetRegisterRow(RegisterRowIndex));
			}
		}
		return R;
	}

	CALL_SITE_DEBUGLOC
	TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> operator != (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> R;

		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) != B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) != B.Registers.GetRegisterRow(1));
		}
		else if(RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) != B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) != B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) != B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) != B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
				for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
				{
					R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) != B.Registers.GetRegisterRow(RegisterRowIndex));
				}
		}
		return R;
	}

	CALL_SITE_DEBUGLOC
		TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> operator == (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B)
	{
		TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> R;

		if(RegisterRowCount == 2)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) == B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) == B.Registers.GetRegisterRow(1));
		}
		else if(RegisterRowCount == 4)
		{
			R.Registers.SetRegisterRow(0, Registers.GetRegisterRow(0) == B.Registers.GetRegisterRow(0));
			R.Registers.SetRegisterRow(1, Registers.GetRegisterRow(1) == B.Registers.GetRegisterRow(1));
			R.Registers.SetRegisterRow(2, Registers.GetRegisterRow(2) == B.Registers.GetRegisterRow(2));
			R.Registers.SetRegisterRow(3, Registers.GetRegisterRow(3) == B.Registers.GetRegisterRow(3));
		}
		else
		{
			UNROLL
				for (uint RegisterRowIndex = 0; RegisterRowIndex < RegisterRowCount; RegisterRowIndex++)
				{
					R.Registers.SetRegisterRow(RegisterRowIndex, Registers.GetRegisterRow(RegisterRowIndex) == B.Registers.GetRegisterRow(RegisterRowIndex));
				}
		}
		return R;
	}

}; // TLaneVector2D


CALL_SITE_DEBUGLOC
template<uint DestVectorSize, typename ScalarType, uint VectorSize, uint SimdSizeX, uint SimdSizeY>
TLaneVector2D<ScalarType, DestVectorSize, SimdSizeX, SimdSizeY> ResizeChannels(
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> A)
{
	TLaneVector2D<ScalarType, DestVectorSize, SimdSizeX, SimdSizeY> R;
	R.SetAllElements(ScalarType(0));
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < SimdSizeX * SimdSizeY; ElementIndex++)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < min(VectorSize, DestVectorSize); ComponentIndex++)
		{
			R.Registers.SetElementComponent(ElementIndex, ComponentIndex, A.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}
	}
	return R;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSizeA, uint VectorSizeB, uint SimdSizeX, uint SimdSizeY>
TLaneVector2D<ScalarType, VectorSizeA + VectorSizeB, SimdSizeX, SimdSizeY> Concatenate(
	TLaneVector2D<ScalarType, VectorSizeA, SimdSizeX, SimdSizeY> A,
	TLaneVector2D<ScalarType, VectorSizeB, SimdSizeX, SimdSizeY> B)
{
	TLaneVector2D<ScalarType, VectorSizeA + VectorSizeB, SimdSizeX, SimdSizeY> R;
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < SimdSizeX * SimdSizeY; ElementIndex++)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeA; ComponentIndex++)
		{
			R.Registers.SetElementComponent(ElementIndex, ComponentIndex, A.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}

		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeB; ComponentIndex++)
		{
			R.Registers.SetElementComponent(ElementIndex, VectorSizeA + ComponentIndex, B.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}
	}
	return R;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSizeA, uint VectorSizeB, uint VectorSizeC, uint SimdSizeX, uint SimdSizeY>
TLaneVector2D<ScalarType, VectorSizeA + VectorSizeB + VectorSizeC, SimdSizeX, SimdSizeY> Concatenate(
	TLaneVector2D<ScalarType, VectorSizeA, SimdSizeX, SimdSizeY> A,
	TLaneVector2D<ScalarType, VectorSizeB, SimdSizeX, SimdSizeY> B,
	TLaneVector2D<ScalarType, VectorSizeC, SimdSizeX, SimdSizeY> C)
{
	TLaneVector2D<ScalarType, VectorSizeA + VectorSizeB + VectorSizeC, SimdSizeX, SimdSizeY> R;
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < SimdSizeX * SimdSizeY; ElementIndex++)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeA; ComponentIndex++)
		{
			R.Registers.SetElementComponent(ElementIndex, ComponentIndex, A.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}

		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeB; ComponentIndex++)
		{
			R.Registers.SetElementComponent(ElementIndex, VectorSizeA + ComponentIndex, B.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}
		
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeC; ComponentIndex++)
		{
			R.Registers.SetElementComponent(ElementIndex, VectorSizeA + VectorSizeB + ComponentIndex, C.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}
	}
	return R;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSizeA, uint VectorSizeB, uint SimdSizeX, uint SimdSizeY>
void Deconcatenate(
	TLaneVector2D<ScalarType, VectorSizeA + VectorSizeB, SimdSizeX, SimdSizeY> M,
	out TLaneVector2D<ScalarType, VectorSizeA, SimdSizeX, SimdSizeY> A,
	out TLaneVector2D<ScalarType, VectorSizeB, SimdSizeX, SimdSizeY> B)
{
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < SimdSizeX * SimdSizeY; ElementIndex++)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeA; ComponentIndex++)
		{
			A.Registers.SetElementComponent(ElementIndex, ComponentIndex, M.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}

		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeB; ComponentIndex++)
		{
			B.Registers.SetElementComponent(ElementIndex, ComponentIndex, M.Registers.GetElementComponent(ElementIndex, VectorSizeA + ComponentIndex));
		}
	}
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSizeA, uint VectorSizeB, uint VectorSizeC, uint SimdSizeX, uint SimdSizeY>
void Deconcatenate(
	TLaneVector2D<ScalarType, VectorSizeA + VectorSizeB + VectorSizeC, SimdSizeX, SimdSizeY> M,
	out TLaneVector2D<ScalarType, VectorSizeA, SimdSizeX, SimdSizeY> A,
	out TLaneVector2D<ScalarType, VectorSizeB, SimdSizeX, SimdSizeY> B,
	out TLaneVector2D<ScalarType, VectorSizeC, SimdSizeX, SimdSizeY> C)
{
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < SimdSizeX * SimdSizeY; ElementIndex++)
	{
		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeA; ComponentIndex++)
		{
			A.Registers.SetElementComponent(ElementIndex, ComponentIndex, M.Registers.GetElementComponent(ElementIndex, ComponentIndex));
		}

		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeB; ComponentIndex++)
		{
			B.Registers.SetElementComponent(ElementIndex, ComponentIndex, M.Registers.GetElementComponent(ElementIndex, VectorSizeA + ComponentIndex));
		}

		UNROLL
		for (uint ComponentIndex = 0; ComponentIndex < VectorSizeC; ComponentIndex++)
		{
			C.Registers.SetElementComponent(ElementIndex, ComponentIndex, M.Registers.GetElementComponent(ElementIndex, VectorSizeA + VectorSizeB + ComponentIndex));
		}
	}
}


// ------------- 1 parameter
	
#define TVECTOR_FUNCTION_1PARAMS_ALIASE_FUNCTION_NAME(ReturnScalarType, FunctionName,AliasFunctionName) \
	CALL_SITE_DEBUGLOC \
	template<typename ScalarType, uint VectorSize, uint SimdSizeX, uint SimdSizeY> \
	TLaneVector2D<ReturnScalarType, VectorSize, SimdSizeX, SimdSizeY> AliasFunctionName( \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> A) \
	{ \
		TLaneVector2D<ReturnScalarType, VectorSize, SimdSizeX, SimdSizeY> R; \
		if(TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount == 2) \
		{ \
			R.Registers.SetRegisterRow(0, FunctionName(A.Registers.GetRegisterRow(0))); \
			R.Registers.SetRegisterRow(1, FunctionName(A.Registers.GetRegisterRow(1))); \
		} \
		else if(TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount == 4) \
		{ \
			R.Registers.SetRegisterRow(0, FunctionName(A.Registers.GetRegisterRow(0))); \
			R.Registers.SetRegisterRow(1, FunctionName(A.Registers.GetRegisterRow(1))); \
			R.Registers.SetRegisterRow(2, FunctionName(A.Registers.GetRegisterRow(2))); \
			R.Registers.SetRegisterRow(3, FunctionName(A.Registers.GetRegisterRow(3))); \
		} \
		else \
		{ \
			UNROLL for (uint RegisterRowIndex = 0; RegisterRowIndex < TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount; RegisterRowIndex++) { \
				R.Registers.SetRegisterRow(RegisterRowIndex, FunctionName(A.Registers.GetRegisterRow(RegisterRowIndex))); \
			} \
		} \
		return R; \
	} \

#define TVECTOR_FUNCTION_1PARAMS(ReturnScalarType, FunctionName) TVECTOR_FUNCTION_1PARAMS_ALIASE_FUNCTION_NAME(ReturnScalarType, FunctionName, FunctionName)

TVECTOR_FUNCTION_1PARAMS(ScalarType, log);
TVECTOR_FUNCTION_1PARAMS(ScalarType, log2);
TVECTOR_FUNCTION_1PARAMS(ScalarType, sqrt);
TVECTOR_FUNCTION_1PARAMS(ScalarType, rsqrt);
TVECTOR_FUNCTION_1PARAMS(ScalarType, exp);
TVECTOR_FUNCTION_1PARAMS(ScalarType, rcp);
TVECTOR_FUNCTION_1PARAMS(ScalarType, saturate);
TVECTOR_FUNCTION_1PARAMS(ScalarType, abs);
TVECTOR_FUNCTION_1PARAMS(ScalarType, floor);
TVECTOR_FUNCTION_1PARAMS(ScalarType, ceil);
TVECTOR_FUNCTION_1PARAMS(ScalarType, round);
TVECTOR_FUNCTION_1PARAMS(ScalarType, fast_sign);
TVECTOR_FUNCTION_1PARAMS(bool, not);
TVECTOR_FUNCTION_1PARAMS(uint, bit_not);
#if PLATFORM_SUPPORTS_REAL_TYPES
	TVECTOR_FUNCTION_1PARAMS(uint16_t, asuint16);
#endif
//Work around: cannot use keyword asuint and asfloat on struct.
TVECTOR_FUNCTION_1PARAMS_ALIASE_FUNCTION_NAME(uint, asuint, azuint);
TVECTOR_FUNCTION_1PARAMS_ALIASE_FUNCTION_NAME(float,asfloat, azfloat);

// ------------- 2 parameters

#define TVECTOR_FUNCTION_2PARAMS(FunctionName) \
	CALL_SITE_DEBUGLOC \
	template<typename ScalarType, uint VectorSize, uint SimdSizeX, uint SimdSizeY> \
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> FunctionName( \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> A, \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B) \
	{ \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R; \
		if (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount == 2) \
		{ \
			R.Registers.SetRegisterRow(0, FunctionName(A.Registers.GetRegisterRow(0), B.Registers.GetRegisterRow(0))); \
			R.Registers.SetRegisterRow(1, FunctionName(A.Registers.GetRegisterRow(1), B.Registers.GetRegisterRow(1))); \
		} \
		else if (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount == 4) \
		{ \
			R.Registers.SetRegisterRow(0, FunctionName(A.Registers.GetRegisterRow(0), B.Registers.GetRegisterRow(0))); \
			R.Registers.SetRegisterRow(1, FunctionName(A.Registers.GetRegisterRow(1), B.Registers.GetRegisterRow(1))); \
			R.Registers.SetRegisterRow(2, FunctionName(A.Registers.GetRegisterRow(2), B.Registers.GetRegisterRow(2))); \
			R.Registers.SetRegisterRow(3, FunctionName(A.Registers.GetRegisterRow(3), B.Registers.GetRegisterRow(3))); \
		} \
		else \
		{ \
			UNROLL for (uint RegisterRowIndex = 0; RegisterRowIndex < TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount; RegisterRowIndex++) { \
				R.Registers.SetRegisterRow(RegisterRowIndex, FunctionName(A.Registers.GetRegisterRow(RegisterRowIndex), B.Registers.GetRegisterRow(RegisterRowIndex))); \
			} \
		} \
		return R; \
	} \

TVECTOR_FUNCTION_2PARAMS(pow);
TVECTOR_FUNCTION_2PARAMS(min);
TVECTOR_FUNCTION_2PARAMS(max);
TVECTOR_FUNCTION_2PARAMS(and_internal);
TVECTOR_FUNCTION_2PARAMS(or_internal);
TVECTOR_FUNCTION_2PARAMS(bit_and);
TVECTOR_FUNCTION_2PARAMS(bit_or);
TVECTOR_FUNCTION_2PARAMS(bit_shift_left);
TVECTOR_FUNCTION_2PARAMS(bit_shift_right);


// ------------- 2 parameters but different return dimension

#define TVECTOR_FUNCTION_2PARAMS_DIFF_RETURN(ReturnVectorSize, FunctionName) \
	CALL_SITE_DEBUGLOC \
	template<typename ScalarType, uint VectorSize, uint SimdSizeX, uint SimdSizeY> \
	TLaneVector2D<ScalarType, ReturnVectorSize, SimdSizeX, SimdSizeY> FunctionName( \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> A, \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B) \
	{ \
		TLaneVector2D<ScalarType, ReturnVectorSize, SimdSizeX, SimdSizeY> R; \
		if (SimdSizeX * SimdSizeY == 2) \
		{ \
			R.SetElement(0, FunctionName(A.GetElement(0), B.GetElement(0))); \
			R.SetElement(1, FunctionName(A.GetElement(1), B.GetElement(1))); \
		} \
		else if (SimdSizeX * SimdSizeY == 4) \
		{ \
			R.SetElement(0, FunctionName(A.GetElement(0), B.GetElement(0))); \
			R.SetElement(1, FunctionName(A.GetElement(1), B.GetElement(1))); \
			R.SetElement(2, FunctionName(A.GetElement(2), B.GetElement(2))); \
			R.SetElement(3, FunctionName(A.GetElement(3), B.GetElement(3))); \
		} \
		else \
		{ \
			UNROLL for (uint ElementIndex = 0; ElementIndex < SimdSizeX * SimdSizeY; ElementIndex++) { \
				R.SetElement(ElementIndex, FunctionName(A.GetElement(ElementIndex), B.GetElement(ElementIndex))); \
			} \
		} \
		return R; \
	} \
	
TVECTOR_FUNCTION_2PARAMS_DIFF_RETURN(/* VectorSize = */ 1, dot);


// ------------- 3 parameters
#define TVECTOR_FUNCTION_3PARAMS(FunctionName, ScalarTypeA) \
	CALL_SITE_DEBUGLOC \
	template<typename ScalarType, uint VectorSize, uint SimdSizeX, uint SimdSizeY> \
	TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> FunctionName( \
		TLaneVector2D<ScalarTypeA, VectorSize, SimdSizeX, SimdSizeY> A, \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> B, \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> C) \
	{ \
		TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY> R; \
		if (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount == 2) \
		{ \
			R.Registers.SetRegisterRow(0, FunctionName(A.Registers.GetRegisterRow(0), B.Registers.GetRegisterRow(0), C.Registers.GetRegisterRow(0))); \
			R.Registers.SetRegisterRow(1, FunctionName(A.Registers.GetRegisterRow(1), B.Registers.GetRegisterRow(1), C.Registers.GetRegisterRow(1))); \
		} \
		else if (TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount == 4) \
		{ \
			R.Registers.SetRegisterRow(0, FunctionName(A.Registers.GetRegisterRow(0), B.Registers.GetRegisterRow(0), C.Registers.GetRegisterRow(0))); \
			R.Registers.SetRegisterRow(1, FunctionName(A.Registers.GetRegisterRow(1), B.Registers.GetRegisterRow(1), C.Registers.GetRegisterRow(1))); \
			R.Registers.SetRegisterRow(2, FunctionName(A.Registers.GetRegisterRow(2), B.Registers.GetRegisterRow(2), C.Registers.GetRegisterRow(2))); \
			R.Registers.SetRegisterRow(3, FunctionName(A.Registers.GetRegisterRow(3), B.Registers.GetRegisterRow(3), C.Registers.GetRegisterRow(3))); \
		} \
		else \
		{ \
			UNROLL for (uint RegisterRowIndex = 0; RegisterRowIndex < TLaneVector2D<ScalarType, VectorSize, SimdSizeX, SimdSizeY>::RegisterRowCount; RegisterRowIndex++) { \
				R.Registers.SetRegisterRow(RegisterRowIndex, FunctionName(A.Registers.GetRegisterRow(RegisterRowIndex), B.Registers.GetRegisterRow(RegisterRowIndex), C.Registers.GetRegisterRow(RegisterRowIndex))); \
			} \
		} \
		return R; \
	} \
	
TVECTOR_FUNCTION_3PARAMS(select_internal, bool);
TVECTOR_FUNCTION_3PARAMS(clamp, ScalarType);
TVECTOR_FUNCTION_3PARAMS(lerp, ScalarType);
TVECTOR_FUNCTION_3PARAMS(min3, ScalarType);
TVECTOR_FUNCTION_3PARAMS(max3, ScalarType);
#if COMPILER_SUPPORTS_MED3
	TVECTOR_FUNCTION_3PARAMS(med3, ScalarType);
#endif


// ------------- AnyElement & AllElement

CALL_SITE_DEBUGLOC
template<uint VectorSize, uint SimdSizeX, uint SimdSizeY>
TLaneVector2D<bool, 1, SimdSizeX, SimdSizeY> AnyElement(TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> A)
{
	TLaneVector2D<bool, 1, SimdSizeX, SimdSizeY> R;
	R.SetElement(0, any(A.GetElement(0)));

	if (SimdSizeX * SimdSizeY == 2)
	{
		R.SetElement(1, any(A.GetElement(1)));
	}	
	else if (SimdSizeX * SimdSizeY == 4)
	{
		R.SetElement(1, any(A.GetElement(1)));
		R.SetElement(2, any(A.GetElement(2)));
		R.SetElement(3, any(A.GetElement(3)));
	}	
	else
	{
		UNROLL for (uint SimdIndex = 1; SimdIndex < SimdSizeX * SimdSizeY; SimdIndex++) { R.SetElement(SimdIndex, any(A.GetElement(SimdIndex))); }
	}

	return R;
}

CALL_SITE_DEBUGLOC
template<uint VectorSize, uint SimdSizeX, uint SimdSizeY>
TLaneVector2D<bool, 1, SimdSizeX, SimdSizeY> AllElement(TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> A)
{
	TLaneVector2D<bool, 1, SimdSizeX, SimdSizeY> R;
	R.SetElement(0, all(A.GetElement(0)));

	if (SimdSizeX * SimdSizeY == 2)
	{
		R.SetElement(1, all(A.GetElement(1)));
	}	
	else if (SimdSizeX * SimdSizeY == 4)
	{
		R.SetElement(1, all(A.GetElement(1)));
		R.SetElement(2, all(A.GetElement(2)));
		R.SetElement(3, all(A.GetElement(3)));
	}	
	else
	{
		UNROLL for (uint SimdIndex = 1; SimdIndex < SimdSizeX * SimdSizeY; SimdIndex++) { R.SetElement(SimdIndex, all(A.GetElement(SimdIndex))); }
	}

	return R;
}

CALL_SITE_DEBUGLOC
template<uint VectorSize, uint SimdSizeX, uint SimdSizeY>
bool AnyComponent(TLaneVector2D<bool, VectorSize, SimdSizeX, SimdSizeY> A)
{
	bool R = any(A.GetElement(0));
	
	if (SimdSizeX * SimdSizeY == 2)
	{
		R = R || any(A.GetElement(1));
	}	
	else if (SimdSizeX * SimdSizeY == 4)
	{
		R = R || any(A.GetElement(1));
		R = R || any(A.GetElement(2));
		R = R || any(A.GetElement(3));
	}	
	else
	{
		UNROLL for (uint SimdIndex = 1; SimdIndex < SimdSizeX * SimdSizeY; SimdIndex++) { R = R || any(A.GetElement(SimdIndex)); }
	}
	return R;
}
