// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define NV_HITOBJECT_USE_MACRO_API

#include "/Engine/Shared/ThirdParty/NVIDIA/nvHLSLExtns.h"
