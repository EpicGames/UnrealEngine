// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OverloadMacros.ush: Macros to conveninently overload standard functions
	for different types.
=============================================================================*/

#pragma once

/** Overloads a function that takes 1 paramater for all vector types */
#define DECLARE_VECTOR_FUNCTION_OVERLOAD_1_PARAM(FunctionName,Type) \
	Type##2 FunctionName(Type##2 v) { return Type##2(FunctionName(v.x),  FunctionName(v.y)); } \
	Type##3 FunctionName(Type##3 v) { return Type##3(FunctionName(v.xy), FunctionName(v.z)); } \
	Type##4 FunctionName(Type##4 v) { return Type##4(FunctionName(v.xy), FunctionName(v.zw)); }
	
/** Overloads a function that takes 2 paramaters for all vector types */
#define DECLARE_VECTOR_FUNCTION_OVERLOAD_2_PARAM(FunctionName,Type) \
	Type##2 FunctionName(Type##2 a, Type##2 b) { return Type##2(FunctionName(a.x , b.x ), FunctionName(a.y , b.y )); } \
	Type##3 FunctionName(Type##3 a, Type##3 b) { return Type##3(FunctionName(a.xy, b.xy), FunctionName(a.z , b.z )); } \
	Type##4 FunctionName(Type##4 a, Type##4 b) { return Type##4(FunctionName(a.xy, b.xy), FunctionName(a.zw, b.zw)); }
	
/** Overloads a function that takes 3 paramaters for all vector types */
#define DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(FunctionName,Type) \
	Type##2 FunctionName(Type##2 a, Type##2 b, Type##2 c) { return Type##2(FunctionName(a.x , b.x , c.x ), FunctionName(a.y , b.y , c.y )); } \
	Type##3 FunctionName(Type##3 a, Type##3 b, Type##3 c) { return Type##3(FunctionName(a.xy, b.xy, c.xy), FunctionName(a.z , b.z , c.z )); } \
	Type##4 FunctionName(Type##4 a, Type##4 b, Type##4 c) { return Type##4(FunctionName(a.xy, b.xy, c.xy), FunctionName(a.zw, b.zw, c.zw)); }
