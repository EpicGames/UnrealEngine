// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Prefix.usf: USF file automatically included by shader preprocessor.
=============================================================================*/

#pragma once

#include "FP16Math.ush"

// Values of FEATURE_LEVEL.
#define FEATURE_LEVEL_ES2_REMOVED	1
#define FEATURE_LEVEL_ES3_1			2
#define FEATURE_LEVEL_SM3			3
#define FEATURE_LEVEL_SM4			4
#define FEATURE_LEVEL_SM5			5
#define FEATURE_LEVEL_SM6			6
#define FEATURE_LEVEL_MAX			7

// Values of PLATFORM_GPU_ARCH
#define PLATFORM_GPU_ARCH_UNKNOWN          0x0000

#define PLATFORM_GPU_ARCH_AMD              0x1000
#define PLATFORM_GPU_ARCH_AMD_GCN_1        0x1010
#define PLATFORM_GPU_ARCH_AMD_GCN_2        0x1020
#define PLATFORM_GPU_ARCH_AMD_GCN_3        0x1030
#define PLATFORM_GPU_ARCH_AMD_GCN_4        0x1040
#define PLATFORM_GPU_ARCH_AMD_GCN_5        0x1050
#define PLATFORM_GPU_ARCH_AMD_RDNA_1       0x1110
#define PLATFORM_GPU_ARCH_AMD_RDNA_2       0x1120
#define PLATFORM_GPU_ARCH_AMD_RDNA_3       0x1130
#define PLATFORM_GPU_ARCH_AMD_LATTEST      PLATFORM_GPU_ARCH_AMD_RDNA_3

#define PLATFORM_GPU_ARCH_NVIDIA           0x2000
#define PLATFORM_GPU_ARCH_NVIDIA_KEPLER    0x2010
#define PLATFORM_GPU_ARCH_NVIDIA_MAXWELL   0x2020
#define PLATFORM_GPU_ARCH_NVIDIA_PASCAL    0x2030
#define PLATFORM_GPU_ARCH_NVIDIA_VOLTA     0x2040
#define PLATFORM_GPU_ARCH_NVIDIA_TURING    0x2050
#define PLATFORM_GPU_ARCH_NVIDIA_AMPERE    0x2060
#define PLATFORM_GPU_ARCH_NVIDIA_ADA       0x2070
#define PLATFORM_GPU_ARCH_NVIDIA_LATTEST   PLATFORM_GPU_ARCH_NVIDIA_ADA

#define PLATFORM_GPU_ARCH_INTEL            0x3000
#define PLATFORM_GPU_ARCH_INTEL_ARC        0x3010
#define PLATFORM_GPU_ARCH_INTEL_LATTEST    PLATFORM_GPU_ARCH_INTEL_ARC


// ---------------------------------------------------- Profile or compiler specific includes
// TODO: Have shader compiler including these platform specific USF files, that needs to work
// with ShaderCore.cpp's GetShaderIncludes().

#ifdef OVERRIDE_PLATFORMCOMMON_USH
	#include "/Platform/Public/PlatformCommon.ush"
#elif COMPILER_METAL
	// Helps with iteration when changing Metal shader code generation backend.
	#include "Platform/Metal/MetalCommon.ush"
#elif COMPILER_VULKAN
	// Helps with iteration when changing Vulkan shader code generation backend.
	#include "Platform/Vulkan/VulkanCommon.ush"
#elif COMPILER_GLSL || COMPILER_GLSL_ES3_1
	// Helps with iteration when changing Vulkan shader code generation backend.
	#include "Platform/GL/GLCommon.ush"
#elif COMPILER_HLSL
	#include "Platform/D3D/D3DCommon.ush"
#endif

#include "/Engine/Public/BindlessResources.ush"
#include "/Engine/Shared/MaterialCacheDefinitions.h"
#include "/Engine/Public/IndirectVirtualTexture.ush"
#include "/Engine/Public/OverloadMacros.ush"


// ---------------------------------------------------- DDC invalidation
// to support the console command "r.InvalidateShaderCache"
#include "ShaderVersion.ush"


// ---------------------------------------------------- COMPILE_* and *_PROFILE defaults

#ifndef COMPILER_HLSLCC
#define COMPILER_HLSLCC 0
#endif

#ifndef COMPILER_DXC
#define COMPILER_DXC 0
#endif

#ifndef COMPILER_FXC
#define COMPILER_FXC 0
#endif

#ifndef COMPILER_HLSL
#define COMPILER_HLSL 0
#endif

#ifndef COMPILER_PSSL
#define COMPILER_PSSL 0
#endif

#ifndef COMPILER_GLSL
#define COMPILER_GLSL 0
#endif

#ifndef COMPILER_GLSL_ES3_1
#define COMPILER_GLSL_ES3_1 0
#endif

#ifndef COMPILER_METAL
#define COMPILER_METAL 0
#endif

#ifndef COMPILER_SUPPORTS_ATTRIBUTES
#define COMPILER_SUPPORTS_ATTRIBUTES 0
#endif

#ifndef COMPILER_SUPPORTS_QUAD_PASS
#define COMPILER_SUPPORTS_QUAD_PASS 0
#endif

#ifndef COMPILER_SUPPORTS_DUAL_SOURCE_BLENDING_SLOT_DECORATION
#define COMPILER_SUPPORTS_DUAL_SOURCE_BLENDING_SLOT_DECORATION	0
#endif

#ifndef COMPILER_SUPPORTS_PRIMITIVE_SHADERS
#define COMPILER_SUPPORTS_PRIMITIVE_SHADERS 0
#endif

#ifndef COMPILER_SUPPORTS_BARYCENTRIC_INTRINSICS
#define COMPILER_SUPPORTS_BARYCENTRIC_INTRINSICS 0
#endif

// Whether the shader compiler supports WAVESIZE()
#ifndef COMPILER_SUPPORTS_WAVE_SIZE
#define COMPILER_SUPPORTS_WAVE_SIZE 0
#endif

#ifndef COMPILER_SUPPORTS_WAVE_MATCH_INTRINSICS
#define COMPILER_SUPPORTS_WAVE_MATCH_INTRINSICS 0
#endif

#ifndef COMPILER_FORCE_WAVE32_MODE
#define COMPILER_FORCE_WAVE32_MODE
#endif

#ifndef COMPILER_ALLOW_CS_DERIVATIVES
#define COMPILER_ALLOW_CS_DERIVATIVES
#endif

#ifndef COMPILER_ALLOW_NON_UNIFORM_DESCRIPTORS
#define COMPILER_ALLOW_NON_UNIFORM_DESCRIPTORS
#endif

#ifndef PLATFORM_REQUIRES_UNWRAPPED_MESH_SHADER_ARGS
#define PLATFORM_REQUIRES_UNWRAPPED_MESH_SHADER_ARGS 0
#endif

#ifndef PLATFORM_SUPPORTS_ROV
#define PLATFORM_SUPPORTS_ROV 0
#endif

#if !PLATFORM_SUPPORTS_ROV
#define RasterizerOrderedTexture2D RWTexture2D
#endif

#ifndef SM6_PROFILE
#define SM6_PROFILE 0
#endif

#ifndef SM5_PROFILE
#define SM5_PROFILE 0
#endif

#ifndef OPENGL_PROFILE
#define OPENGL_PROFILE 0
#endif

#ifndef ES3_1_PROFILE
#define ES3_1_PROFILE 0
#endif

#ifndef METAL_ES3_1_PROFILE
#define METAL_ES3_1_PROFILE 0
#endif

// Deprecated, use METAL_ES3_1_PROFILE instead
#ifndef METAL_PROFILE
#define METAL_PROFILE METAL_ES3_1_PROFILE
#endif

#ifndef METAL_SM5_IOS_TVOS_PROFILE
#define METAL_SM5_IOS_TVOS_PROFILE 0
#endif

// Deprecated, use METAL_SM5_IOS_TVOS_PROFILE instead
#ifndef METAL_MRT_PROFILE
#define METAL_MRT_PROFILE METAL_SM5_IOS_TVOS_PROFILE
#endif

#ifndef METAL_SM5_PROFILE
#define METAL_SM5_PROFILE 0
#endif

#ifndef METAL_SM6_PROFILE
#define METAL_SM6_PROFILE 0
#endif

#ifndef COMPILER_VULKAN
#define	COMPILER_VULKAN 0
#endif

#ifndef VULKAN_PROFILE
#define	VULKAN_PROFILE 0
#endif

#ifndef VULKAN_PROFILE_SM5
#define	VULKAN_PROFILE_SM5 0
#endif

#ifndef VULKAN_PROFILE_SM6
#define	VULKAN_PROFILE_SM6 0
#endif

#ifndef IOS
#define IOS 0
#endif

#ifndef MAC
#define MAC 0
#endif

#ifndef VECTORVM_PROFILE
#define VECTORVM_PROFILE 0
#endif

#ifndef IR_LANGUAGE_DXBC
#define IR_LANGUAGE_DXBC 0
#endif

// 'static' asserts
#if COMPILER_GLSL || COMPILER_GLSL_ES3_1 || COMPILER_VULKAN || COMPILER_METAL
#if !COMPILER_HLSLCC
#error "Missing COMPILER_HLSLCC define!"
#endif
#endif

#ifndef PLATFORM_SUPPORTS_CALLABLE_SHADERS
#define PLATFORM_SUPPORTS_CALLABLE_SHADERS 0
#endif

// Whether the platforms support official SM6 wave intrinsics
// https://github.com/Microsoft/DirectXShaderCompiler/wiki/Wave-Intrinsics
#ifndef PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
#define PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS 0
#endif

#ifndef PLATFORM_SUPPORTS_REAL_TYPES
#define PLATFORM_SUPPORTS_REAL_TYPES 0
#endif

#ifndef PLATFORM_SUPPORTS_RELAXED_PRECISION
#define PLATFORM_SUPPORTS_RELAXED_PRECISION 0
#endif

#ifndef COMPILER_SUPPORTS_WAVE_32_64_MODE
#define COMPILER_SUPPORTS_WAVE_32_64_MODE 0
#endif

//Platforms that don't run the editor shouldn't need editor features in the shaders.
#ifndef PLATFORM_SUPPORTS_EDITOR_SHADERS
#define PLATFORM_SUPPORTS_EDITOR_SHADERS 1
#endif

#ifndef COMPILER_SUPPORTS_HLSL2021
#define COMPILER_SUPPORTS_HLSL2021 0
#endif

#ifndef PLATFORM_SUPPORTS_CONSTANTBUFFER_OBJECT
#define PLATFORM_SUPPORTS_CONSTANTBUFFER_OBJECT 0
#endif

#ifndef PLATFORM_SUPPORTS_UNIFORM_BUFFER_OBJECTS
#define PLATFORM_SUPPORTS_UNIFORM_BUFFER_OBJECTS 0
#endif

#ifndef PLATFORM_GPU_ARCH
#define PLATFORM_GPU_ARCH PLATFORM_GPU_ARCH_UNKNOWN
#endif 

#ifndef PLATFORM_NEEDS_DEPTH_TEXTURE_READS
#define PLATFORM_NEEDS_DEPTH_TEXTURE_READS 0
#endif

#ifndef PLATFORM_NEEDS_SELECT_UINT
#define PLATFORM_NEEDS_SELECT_UINT 0
#endif

#ifndef PLATFORM_NEEDS_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE
#define PLATFORM_NEEDS_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE 0
#endif

#ifndef PLATFORM_SUPPORTS_SUBSTRATE_UINT1
#define PLATFORM_SUPPORTS_SUBSTRATE_UINT1 1
#endif

#ifndef COMPILER_NEEDS_DETERMINANT
#define COMPILER_NEEDS_DETERMINANT 0
#endif

#if COMPILER_DXC == 1

#define SHADER_PUSH_WARNINGS_STATE  _Pragma("dxc diagnostic push")
#define SHADER_POP_WARNINGS_STATE   _Pragma("dxc diagnostic pop")

// DXC requires several pragmas because there isn't a warning group that covers all cases
#define SHADER_DISABLE_WARNINGS 								\
	_Pragma("dxc diagnostic ignored \"-Wall\"") 				\
	_Pragma("dxc diagnostic ignored \"-Wconversion\"") 			\
	_Pragma("dxc diagnostic ignored \"-Wfor-redefinition\"") 	\
	_Pragma("dxc diagnostic ignored \"-Winline-asm\"")			\
	_Pragma("dxc diagnostic ignored \"-Wunsequenced\"")

#endif

// If the shader compiler does not suport disabling warnings locally, these macros will be empty
#ifndef SHADER_DISABLE_WARNINGS
#define SHADER_DISABLE_WARNINGS
#endif

#ifndef SHADER_PUSH_WARNINGS_STATE
#define SHADER_PUSH_WARNINGS_STATE
#endif

#ifndef SHADER_POP_WARNINGS_STATE
#define SHADER_POP_WARNINGS_STATE
#endif

// ---------------------------------------------------- Alternative floating point types

#ifndef FORCE_FLOATS
#define FORCE_FLOATS 0
#endif

#if COMPILER_DXC && !FORCE_FLOATS && (ES3_1_PROFILE || METAL_ES3_1_PROFILE || VULKAN_PROFILE_SM5)
	#pragma dxc diagnostic ignored "-Wconversion"
#endif

#if !FORCE_FLOATS && PLATFORM_SUPPORTS_RELAXED_PRECISION
	#define half min16float
	#define half1 min16float1
	#define half2 min16float2
	#define half3 min16float3
	#define half4 min16float4
	#define half3x3 min16float3x3
	#define half3x4 min16float3x4
	#define half4x4 min16float4x4
#elif FORCE_FLOATS || !PLATFORM_SUPPORTS_REAL_TYPES
	#define half float
	#define half1 float1
	#define half2 float2
	#define half3 float3
	#define half4 float4
	#define half3x3 float3x3
	#define half4x4 float4x4 
	#define half4x3 float4x3 
	#define fixed float
	#define fixed1 float1
	#define fixed2 float2
	#define fixed3 float3
	#define fixed4 float4
	#define fixed3x3 float3x3
	#define fixed4x4 float4x4
	#define fixed4x3 float4x3
#endif

// ---------------------------------------------------- Profile config


#if SM6_PROFILE
	// SM6 = full dx12 features (high end UE5 rendering)
	#define FEATURE_LEVEL FEATURE_LEVEL_SM6

#elif SM5_PROFILE
	// SM5 = full dx11 features (high end UE4 rendering)
	#define FEATURE_LEVEL FEATURE_LEVEL_SM5

#elif SWITCH_PROFILE || SWITCH_PROFILE_FORWARD
	#undef ES3_1_PROFILE

	#if SWITCH_PROFILE
		#define FEATURE_LEVEL FEATURE_LEVEL_SM5
	#else
		#define FEATURE_LEVEL FEATURE_LEVEL_ES3_1
		// @todo switch: maybe all uses of this should check feature level not profile?
		#define ES3_1_PROFILE 1
	#endif

#elif VULKAN_PROFILE
	#define FEATURE_LEVEL FEATURE_LEVEL_ES3_1
	// @todo: replace usage of ES3_1_PROFILE with FEATURE_LEVEL where appropriate
	#undef ES3_1_PROFILE
	#define ES3_1_PROFILE 1 

#elif VULKAN_PROFILE_SM5
	#define FEATURE_LEVEL FEATURE_LEVEL_SM5
	#define STENCIL_COMPONENT_SWIZZLE .x

#elif VULKAN_PROFILE_SM6
	#define FEATURE_LEVEL FEATURE_LEVEL_SM6
	#define STENCIL_COMPONENT_SWIZZLE .x

#elif METAL_ES3_1_PROFILE
	#define FEATURE_LEVEL  FEATURE_LEVEL_ES3_1
	// @todo metal: remove this and make sure all uses handle METAL_ES3_1_PROFILE
	#undef ES3_1_PROFILE
	#define ES3_1_PROFILE 1
	#define FCOLOR_COMPONENT_SWIZZLE .rgba
	#define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
	#define STENCIL_COMPONENT_SWIZZLE .x

#elif METAL_SM5_IOS_TVOS_PROFILE
	#define FEATURE_LEVEL FEATURE_LEVEL_SM5
	#define FCOLOR_COMPONENT_SWIZZLE .rgba
	#define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
	#define STENCIL_COMPONENT_SWIZZLE .x

#elif METAL_SM5_PROFILE
	#define FEATURE_LEVEL FEATURE_LEVEL_SM5
	#define FCOLOR_COMPONENT_SWIZZLE .rgba
	#define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
	#define STENCIL_COMPONENT_SWIZZLE .x
 
#elif METAL_SM6_PROFILE
    #define FEATURE_LEVEL FEATURE_LEVEL_SM6
    #define FCOLOR_COMPONENT_SWIZZLE .rgba
    #define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
    #define STENCIL_COMPONENT_SWIZZLE .x

#elif ES3_1_PROFILE
	#define FEATURE_LEVEL FEATURE_LEVEL_ES3_1

	#if COMPILER_GLSL_ES3_1
		#define FCOLOR_COMPONENT_SWIZZLE .bgra  
		#define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
	#else 
		#define FCOLOR_COMPONENT_SWIZZLE .rgba
		#define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
	#endif

	#if COMPILER_GLSL || COMPILER_GLSL_ES3_1
		// A8 textures when sampled have their component in R
		#define A8_SAMPLE_MASK .r
	#endif
#elif VECTORVM_PROFILE
	#define FEATURE_LEVEL FEATURE_LEVEL_SM5
#endif

#ifndef FEATURE_LEVEL
	#error FEATURE_LEVEL has not been defined for this platform. Add it to Platform.ush or in the Common.ush file for this platform

	#define FEATURE_LEVEL FEATURE_LEVEL_MAX

#endif

#if COMPILER_METAL
	// Metal does not allow writes to A8 textures so we are faking it by making them all R8.
	// WARNING: If this changes or the type in MetalRHI changes both must be updated!
	#define A8_SAMPLE_MASK .r
#endif


// ---------------------------------------------------- Swizzle defaults

// If we didn't request color component swizzling, just make it empty
#ifndef FCOLOR_COMPONENT_SWIZZLE
#define FCOLOR_COMPONENT_SWIZZLE .rgba
#endif

#ifndef FMANUALFETCH_COLOR_COMPONENT_SWIZZLE
#define FMANUALFETCH_COLOR_COMPONENT_SWIZZLE .bgra
#endif

#ifndef STENCIL_COMPONENT_SWIZZLE
#define STENCIL_COMPONENT_SWIZZLE .g
#endif

#ifndef A8_SAMPLE_MASK
#define A8_SAMPLE_MASK .a
#endif

// ---------------------------------------------------- Platform dependent supports

// Type macros for Uniform Buffer code generation.
// Hides the platform specific 'half' and 'fixed' support but has to extend to all types and dimensions
#if (FEATURE_LEVEL == FEATURE_LEVEL_ES3_1)
	#define UB_INT(Dim)         int##Dim
	#define UB_UINT(Dim)        uint##Dim
	#define UB_FLOAT(Dim)       float##Dim
	#if METAL_ES3_1_PROFILE || COMPILER_HLSL
		#define UB_HALF_FLOAT(Dim)  float##Dim
	#else
		#define UB_HALF_FLOAT(Dim)  half##Dim
	#endif
	#define UB_FIXED_FLOAT(Dim) fixed##Dim
#else
	#define UB_INT(Dim)         int##Dim
	#define UB_UINT(Dim)        uint##Dim
	#define UB_FLOAT(Dim)       float##Dim
	#define UB_HALF_FLOAT(Dim)  float##Dim
	#define UB_FIXED_FLOAT(Dim) float##Dim
#endif

// 16KB by default. Must match Platform.h
#ifndef PLATFORM_MAX_UNIFORM_BUFFER_RANGE
	#define PLATFORM_MAX_UNIFORM_BUFFER_RANGE (16u*1024u)
#endif
#define PLATFORM_MAX_UNIFORM_BUFFER_RANGE_FLOAT4 (PLATFORM_MAX_UNIFORM_BUFFER_RANGE/16u)

// These macros are used by CreateHLSLUniformBufferDeclaration and the functions it calls to generate shader code for Uniform Buffers.
// Depending on certain scenarios, the UB_CB_ prefixed macros can be swapped out for the UB_CBO_ versions to allow some platforms to define Uniform Buffers differently

#define UB_CB_DEFINITION_START(      UBName                    ) cbuffer UBName {
#define UB_CB_DEFINITION_START_REG(  UBName, Register, Space   ) cbuffer UBName : register(b##Register, space##Space) {
#define UB_CB_MEMBER_NAME(           UBName,         MemberName) UBName##_##MemberName
#define UB_CB_PREFIXED_MEMBER_NAME(  UBName, Prefix, MemberName) Prefix##UBName##_##MemberName
#define UB_CB_DEFINITION_END(        UBName                    ) }
#define UB_CB_FINALIZE(              UBName                    )
#define UB_CB_FINALIZE_REG(          UBName, Register, Space   )
#define UB_CB_MEMBER_ACCESS(         UBName,         MemberName) UBName##_##MemberName
#define UB_CB_PREFIXED_MEMBER_ACCESS(UBName, Prefix, MemberName) Prefix##UBName##_##MemberName

#define UB_CB_REMAP_PARAMETER(UBName, StructName, GlobalName) UBName##.##StructName = UBName##_##GlobalName
#define UB_CB_REMAP_RESOURCE(UBName, StructName, GlobalName)  UBName##.##StructName = UBName##_##GlobalName

#define UB_CB_UNIFORM_BLOCK(UBName, MemberName) UB_FLOAT(4) UB_CB_MEMBER_NAME(UBName,MemberName)[PLATFORM_MAX_UNIFORM_BUFFER_RANGE_FLOAT4]

#if PLATFORM_SUPPORTS_CONSTANTBUFFER_OBJECT
	#define UB_CBO_TYPE(UBName) F##UBName##Constants

	#define UB_CBO_DEFINITION_START(      UBName                    ) struct UB_CBO_TYPE(UBName) {
	#define UB_CBO_DEFINITION_START_REG(  UBName, Register, Space   ) UB_CBO_DEFINITION_START(UBName)
	#define UB_CBO_MEMBER_NAME(           UBName,         MemberName) MemberName
	#define UB_CBO_PREFIXED_MEMBER_NAME(  UBName, Prefix, MemberName) Prefix##MemberName
	#define UB_CBO_DEFINITION_END(        UBName                    ) };
	#define UB_CBO_FINALIZE(              UBName                    ) ConstantBuffer<UB_CBO_TYPE(UBName)> UBName;
	#define UB_CBO_FINALIZE_REG(          UBName, Register, Space   ) ConstantBuffer<UB_CBO_TYPE(UBName)> UBName : register(b##Register, space##Space);
	#define UB_CBO_MEMBER_ACCESS(         UBName,         MemberName) UBName##.##MemberName
	#define UB_CBO_PREFIXED_MEMBER_ACCESS(UBName, Prefix, MemberName) UBName##.##Prefix##MemberName

	#define UB_CBO_REMAP_PARAMETER(UBName, StructName, GlobalName) UBName##.##StructName = UBName##.##GlobalName
	#define UB_CBO_REMAP_RESOURCE(UBName, StructName, GlobalName)  UBName##.##StructName = UBName##_##GlobalName

	#define UB_CBO_UNIFORM_BLOCK(UBName, MemberName) UB_FLOAT(4) UB_CBO_MEMBER_NAME(UBName,MemberName)[PLATFORM_MAX_UNIFORM_BUFFER_RANGE_FLOAT4]

#else
	#define UB_CBO_DEFINITION_START(      UBName                    ) UB_CB_DEFINITION_START(UBName)
	#define UB_CBO_DEFINITION_START_REG(  UBName, Register, Space   ) UB_CB_DEFINITION_START_REG(UBName,Register,Space)
	#define UB_CBO_MEMBER_NAME(           UBName,         MemberName) UB_CB_MEMBER_NAME(UBName,MemberName)
	#define UB_CBO_PREFIXED_MEMBER_NAME(  UBName, Prefix, MemberName) UB_CB_PREFIXED_MEMBER_NAME(UBName,Prefix,MemberName)
	#define UB_CBO_DEFINITION_END(        UBName                    ) UB_CB_DEFINITION_END(UBName)
	#define UB_CBO_FINALIZE(              UBName                    ) UB_CB_FINALIZE(UBName)
	#define UB_CBO_FINALIZE_REG(          UBName, Register, Space   ) UB_CB_FINALIZE_REG(UBName,Register,Space)
	#define UB_CBO_MEMBER_ACCESS(         UBName,         MemberName) UB_CB_MEMBER_ACCESS(UBName,MemberName)
	#define UB_CBO_PREFIXED_MEMBER_ACCESS(UBName, Prefix, MemberName) UB_CB_PREFIXED_MEMBER_ACCESS(UBName,Prefix,MemberName)

	#define UB_CBO_REMAP_PARAMETER(UBName, StructName, GlobalName) UB_CB_REMAP_PARAMETER(UBName,StructName,GlobalName)
	#define UB_CBO_REMAP_RESOURCE( UBName, StructName, GlobalName) UB_CB_REMAP_RESOURCE(UBName,StructName,GlobalName)

	#define UB_CBO_UNIFORM_BLOCK(UBName, MemberName) UB_CB_UNIFORM_BLOCK(UBName, MemberName)
#endif

// Use interpolator for platforms that do not support ClipDistance
#if !PLATFORM_SUPPORTS_CLIP_DISTANCE
#define SV_ClipDistance OUTCLIPDIST
#endif

// non-editor platforms generally never want development/editor features.
#ifndef PLATFORM_SUPPORTS_DEVELOPMENT_SHADERS
#define PLATFORM_SUPPORTS_DEVELOPMENT_SHADERS 1
#endif

#ifndef MOBILE_EMULATION
#define MOBILE_EMULATION 0
#endif

// Whether the platform supports independent texture and samplers (defined in DDSPI)
// When enabled, different texture lookups can share samplers to allow more artist samplers in the base pass
// Ideally this would just be enabled for all SM4 and above feature level platforms
#ifndef SUPPORTS_INDEPENDENT_SAMPLERS
#define SUPPORTS_INDEPENDENT_SAMPLERS 0
#endif

// Whether the platform support pixel coverage on MSAA targets (SV_Coverage).
#define SUPPORTS_PIXEL_COVERAGE (FEATURE_LEVEL >= FEATURE_LEVEL_SM5 && !COMPILER_GLSL && !MOBILE_EMULATION)

// Must match C++ RHISupports4ComponentUAVReadWrite
// D3D11 does not support multi-component loads from a UAV: "error X3676: typed UAV loads are only allowed for single-component 32-bit element types"
#ifndef PLATFORM_SUPPORTS_4COMPONENT_UAV_READ_WRITE
#define PLATFORM_SUPPORTS_4COMPONENT_UAV_READ_WRITE (XBOXONE_PROFILE || COMPILER_METAL)
#endif

// Whether the platform supports binding SRVs to the vertex shader stage.
// This is generally available on high-end platforms and not available on mobile outside of some exceptions.
// The real value is expected to be set based on DataDrivenShaderPlatformInfo in GlobalBeginCompileShader().
#ifndef PLATFORM_SUPPORTS_VERTEX_SHADER_SRVS
#define PLATFORM_SUPPORTS_VERTEX_SHADER_SRVS 1
#endif

// Whether the platform supports binding UAVs to the vertex shader stage.
#ifndef PLATFORM_SUPPORTS_VERTEX_SHADER_UAVS
#define PLATFORM_SUPPORTS_VERTEX_SHADER_UAVS 0
#endif

#ifndef PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
#define PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS 0
#endif


// ---------------------------------------------------- Compiler specific defaults and fallbacks

#if !defined(PLATFORM_BREAK)
	#define PLATFORM_BREAK() 
#endif

#if !defined(PLATFORM_ASSERT)
	#define PLATFORM_ASSERT(condition, assert_id)
	#define PLATFORM_ASSERT1(condition, assert_id, a)
	#define PLATFORM_ASSERT2(condition, assert_id, a, b)
	#define PLATFORM_ASSERT3(condition, assert_id, a, b, c)
	#define PLATFORM_ASSERT4(condition, assert_id, a, b, c, d)
#endif

#if !defined(PLATFORM_SUPPORTS_SHADER_TIMESTAMP)
#if USE_NVAPI_TIMESTAMP
	#include "NvidiaExtensionsCommon.ush"

	#define PLATFORM_SUPPORTS_SHADER_TIMESTAMP 1

	#define FTimestamp uint

	FTimestamp GetShaderTimestamp()
	{
		return NvGetSpecial( NV_SPECIALOP_GLOBAL_TIMER_LO );
	}

	uint ShaderTimestampDiff(FTimestamp TimeBegin, FTimestamp TimeEnd)
	{
	  // Account for (at most one) overflow
	  return TimeEnd >= TimeBegin ? (TimeEnd - TimeBegin) : (~0u - (TimeBegin - TimeEnd));
	}
#else // !USE_NVAPI_TIMESTAMP
	#define PLATFORM_SUPPORTS_SHADER_TIMESTAMP 0
#endif // !USE_NVAPI_TIMESTAMP
#endif // !defined(PLATFORM_SUPPORTS_SHADER_TIMESTAMP)

// Hlslcc platforms ignore the uniform keyword as it can't properly optimize flow
#if COMPILER_HLSLCC
#define uniform
#endif

#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
#if !defined(COMPILER_SUPPORTS_WAVE_ONCE)
	#define COMPILER_SUPPORTS_WAVE_ONCE		 1
#endif
#if !defined(COMPILER_SUPPORTS_WAVE_VOTE)
    #define COMPILER_SUPPORTS_WAVE_VOTE		 1
#endif
#if !defined(COMPILER_SUPPORTS_WAVE_MINMAX)
	#define COMPILER_SUPPORTS_WAVE_MINMAX	 1
#endif
#if !defined(COMPILER_SUPPORTS_WAVE_BIT_ORAND)
	#define COMPILER_SUPPORTS_WAVE_BIT_ORAND 1
#endif
#endif

// If compiler lane management in a wave.
//	WaveGetLaneCount()
//	WaveGetLaneIndex()
//	if (WaveIsFirstLane()) { ... }
#ifndef COMPILER_SUPPORTS_WAVE_ONCE
#define COMPILER_SUPPORTS_WAVE_ONCE 0
#endif

// Whether the compiler exposes voting on all lanes:
//	WaveActiveAnyTrue(MyBool)
//	WaveActiveAnyTrue(MyBool)
//	WaveActiveAllEqual(MyBool)
#ifndef COMPILER_SUPPORTS_WAVE_VOTE
#define COMPILER_SUPPORTS_WAVE_VOTE 0
#endif

// Whether the compiler exposes min max instructions across all lane of the wave.
//	WaveActiveMin(MyFloat)
//	WaveActiveMin(MyInt)
//	WaveActiveMin(MyUint)
//	WaveActiveMax(MyFloat)
//	WaveActiveMax(MyInt)
//	WaveActiveMax(MyUint)
#ifndef COMPILER_SUPPORTS_WAVE_MINMAX
#define COMPILER_SUPPORTS_WAVE_MINMAX 0
#endif

// Whether the compiler exposes OR and AND bit operation all lanes:
//	WaveActiveBitAnd(MyMask)
//	WaveActiveBitOr(MyMask)
#ifndef COMPILER_SUPPORTS_WAVE_BIT_ORAND
#define COMPILER_SUPPORTS_WAVE_BIT_ORAND 0
#endif

// Whether the compiler exposes GCN's ds_swizzle_b32 instruction.
//	float WaveLaneSwizzleGCN(float x, const uint and_mask, const uint or_mask, const uint xor_mask)
#ifndef COMPILER_SUPPORTS_WAVE_SWIZZLE_GCN
#define COMPILER_SUPPORTS_WAVE_SWIZZLE_GCN 0
#endif

#ifndef COMPILER_SUPPORTS_WAVE_PERMUTE
#define COMPILER_SUPPORTS_WAVE_PERMUTE 0
#endif

#ifndef COMPILER_SUPPORTS_WAVE_READCROSSLANE
#define COMPILER_SUPPORTS_WAVE_READCROSSLANE 0
#endif

// Mirrors GRHISupportsPrimitiveShaders.
#ifndef COMPILER_SUPPORTS_PRIMITIVE_SHADERS
#define COMPILER_SUPPORTS_PRIMITIVE_SHADERS 0
#endif

// Mirrors GRHISupportsRectTopology.
#ifndef PLATFORM_SUPPORTS_RECT_LIST
#define PLATFORM_SUPPORTS_RECT_LIST 0
#endif

// Mirrors GRHISupportsAtomicUInt64.
#ifndef PLATFORM_SUPPORTS_ATOMIC_UINT64
#define PLATFORM_SUPPORTS_ATOMIC_UINT64 0
#endif

// Support for depth test running both before and after pixel shader
#ifndef COMPILER_SUPPORTS_DEPTHSTENCIL_EARLYTEST_LATEWRITE
#define COMPILER_SUPPORTS_DEPTHSTENCIL_EARLYTEST_LATEWRITE 0
#endif

#ifndef COMPILER_SUPPORTS_SHADER_YIELD
#define COMPILER_SUPPORTS_SHADER_YIELD 0
void ShaderYield()
{
	// Do nothing
}
#endif

#ifndef COMPILER_SUPPORTS_GATHER_LOD_RED
#define COMPILER_SUPPORTS_GATHER_LOD_RED 0
#endif

#ifndef COMPILER_SUPPORTS_GATHER_UINT
#define COMPILER_SUPPORTS_GATHER_UINT 0
#endif

#ifndef COMPILER_SUPPORTS_MED3
#define COMPILER_SUPPORTS_MED3 0
#endif

#if (ES3_1_PROFILE && !METAL_ES3_1_PROFILE) || VULKAN_PROFILE_SM5
#define HALF_TYPE half
#define HALF2_TYPE half2
#define HALF3_TYPE half3
#define HALF4_TYPE half4
#else
#define HALF_TYPE float
#define HALF2_TYPE float2
#define HALF3_TYPE float3
#define HALF4_TYPE float4
#endif

// ---------------------------------------------------- Compiler attributes

#if SM6_PROFILE || SM5_PROFILE || COMPILER_SUPPORTS_ATTRIBUTES

/** Avoids flow control constructs. */
#define UNROLL [unroll]
#define UNROLL_N(N) [unroll(N)]

/** Gives preference to flow control constructs. */
#define LOOP [loop]

/** Performs branching by using control flow instructions like jmp and label. */
#define BRANCH [branch]

/** Performs branching by using the cnd instructions. */
#define FLATTEN [flatten]

/** Allows a compute shader loop termination condition to be based off of a UAV read. The loop must not contain synchronization intrinsics. */
#define ALLOW_UAV_CONDITION [allow_uav_condition]

#endif // SM6_PROFILE || SM5_PROFILE || COMPILER_SUPPORTS_ATTRIBUTES


#ifndef EARLYDEPTHSTENCIL // Might have been already defined in PlatformCommon.ush
#if SM6_PROFILE || SM5_PROFILE || METAL_SM5_IOS_TVOS_PROFILE || METAL_SM5_PROFILE || METAL_SM6_PROFILE || ES3_1_PROFILE || VULKAN_PROFILE_SM5 || VULKAN_PROFILE_SM6
#define EARLYDEPTHSTENCIL [earlydepthstencil]
#endif
#endif // EARLYDEPTHSTENCIL

#if COMPILER_SUPPORTS_DUAL_SOURCE_BLENDING_SLOT_DECORATION
#define DUAL_SOURCE_BLENDING_SLOT(SLOT) [[vk::location(0), vk::index(SLOT)]]
#endif

// ---------------------------------------------------- Compiler attribute fallbacks

#ifndef UNROLL
#define UNROLL
#endif

#ifndef UNROLL_N
#define UNROLL_N(N)
#endif

#ifndef LOOP
#define LOOP
#endif

#ifndef BRANCH
#define BRANCH
#endif

#ifndef FLATTEN
#define FLATTEN
#endif

#ifndef ALLOW_UAV_CONDITION
#define ALLOW_UAV_CONDITION
#endif

#ifndef INVARIANT
precise float MakePrecise(in precise float v) { precise float pv = v; return pv; }
precise float2 MakePrecise(in precise float2 v) { precise float2 pv = v; return pv; }
precise float3 MakePrecise(in precise float3 v) { precise float3 pv = v; return pv; }
precise float4 MakePrecise(in precise float4 v) { precise float4 pv = v; return pv; }
#define INVARIANT(X) MakePrecise(X)
#endif

#ifndef INVARIANT_ADD
#define INVARIANT_ADD(Lhs, Rhs) INVARIANT((Lhs) + (Rhs))
#endif

#ifndef INVARIANT_SUB
#define INVARIANT_SUB(Lhs, Rhs) INVARIANT((Lhs) - (Rhs))
#endif

#ifndef INVARIANT_MUL
#define INVARIANT_MUL(Lhs, Rhs) INVARIANT((Lhs) * (Rhs))
#endif

#ifndef INVARIANT_DIV
#define INVARIANT_DIV(Lhs, Rhs) INVARIANT((Lhs) / (Rhs))
#endif

#ifndef INVARIANT_FMA
#define INVARIANT_FMA(A, B, C) INVARIANT(FMA((A), (B), (C)))
#endif

#ifndef INVARIANT_OUTPUT
#define INVARIANT_OUTPUT
#endif

#ifndef PLATFORM_SUPPORTS_FMA
#define PLATFORM_SUPPORTS_FMA 1
#define FMA(a, b, c) mad(a, b, c)
#endif

#ifndef ENABLE_RE_Z
#define ENABLE_RE_Z
#endif

#ifndef COMPILER_SUPPORTS_NOINLINE
#define COMPILER_SUPPORTS_NOINLINE 0
#endif

// Informs compiler we want a subroutine created, which can be used to 
// decrease register pressure in certain situations. Code is kept separate,
// and a set number of registers are used on each call. Should only be used
// with extensive profiling, as the default inlining behavior is usually best.
// DXIL:  https://github.com/microsoft/DirectXShaderCompiler/blob/master/tools/clang/test/HLSLFileCheck/hlsl/functions/attribute/noinline.hlsl
// SPIRV: https://www.khronos.org/registry/spir-v/specs/unified1/SPIRV.html (DontInline)
#if COMPILER_SUPPORTS_NOINLINE
#define NOINLINE [noinline]
#else
#define NOINLINE
#endif

#ifndef EARLYDEPTHSTENCIL
#define EARLYDEPTHSTENCIL
#endif

#ifndef DUAL_SOURCE_BLENDING_SLOT
#define DUAL_SOURCE_BLENDING_SLOT(SLOT)
#endif

#ifndef DEPTHSTENCIL_EARLYTEST_LATEWRITE
#define DEPTHSTENCIL_EARLYTEST_LATEWRITE
#endif

#ifndef STRONG_TYPE
#define STRONG_TYPE
#endif

#ifndef StrongTypedBuffer
#define StrongTypedBuffer Buffer
#endif

#ifndef RWCoherentBuffer
#define RWCoherentBuffer(TYPE) globallycoherent RWBuffer<TYPE>
#endif

#ifndef RWCoherentStructuredBuffer
#define RWCoherentStructuredBuffer(TYPE) globallycoherent RWStructuredBuffer<TYPE>
#endif

// Drops globallycoherent qualifier when used in a struct
#ifndef RWCoherentStructuredBufferRef
#define RWCoherentStructuredBufferRef(TYPE) RWStructuredBuffer<TYPE>
#endif

#ifndef RWCoherentByteAddressBuffer
#define RWCoherentByteAddressBuffer globallycoherent RWByteAddressBuffer
#endif

// Drops globallycoherent qualifier when used in a struct
#ifndef RWCoherentByteAddressBufferRef
#define RWCoherentByteAddressBufferRef RWByteAddressBuffer
#endif

// Flag to say if the compiler needs globallycoherent locals of RWCoherentByteAddressBufferRef/etc
#ifndef COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
#define COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS 0
#endif

#ifndef ISOLATE
#define ISOLATE
#endif

#ifndef HOIST_DESCRIPTORS
#define HOIST_DESCRIPTORS
#endif

#ifndef CALL_SITE_DEBUGLOC
#define CALL_SITE_DEBUGLOC
#endif

#ifndef NO_LICM
#define NO_LICM
#endif

#ifndef SCHEDULER_MIN_PRESSURE
#define SCHEDULER_MIN_PRESSURE
#endif

#ifndef MAX_OCCUPANCY
#define MAX_OCCUPANCY
#endif

#ifndef DISABLE_TARGET_OCCUPANCY_WARNING
#define DISABLE_TARGET_OCCUPANCY_WARNING
#endif

#ifndef DISABLE_POTENTIALLY_UNINITIALIZED_WARNING
#define DISABLE_POTENTIALLY_UNINITIALIZED_WARNING
#endif

#ifndef ALLOW_NO_PS_EXPORT
#define ALLOW_NO_PS_EXPORT
#endif

#ifndef ADAPTIVE_LICM
#define ADAPTIVE_LICM
#endif

// ---------------------------------------------------- Interpolator attribute fallbacks

#ifndef COMPRESSED_16_FLOAT
#define COMPRESSED_16_FLOAT
#endif

#ifndef COMPRESSED_16_UNORM
#define COMPRESSED_16_UNORM
#endif

#ifndef COMPRESSED_16_SNORM
#define COMPRESSED_16_SNORM
#endif

#ifndef COMPRESSED_16_UINT
#define COMPRESSED_16_UINT
#endif

#ifndef COMPRESSED_16_INT
#define COMPRESSED_16_INT
#endif

#ifndef COMPRESSED_8_UNORM
#define COMPRESSED_8_UNORM
#endif

#ifndef COMPRESSED_8_SNORM
#define COMPRESSED_8_SNORM
#endif

#ifndef COMPRESSED_8_UINT
#define COMPRESSED_8_UINT
#endif

#ifndef CUSTOM_INTERPOLATION
#define CUSTOM_INTERPOLATION nointerpolation
#endif

// ---------------------------------------------------- Global uses

#define USE_DEVELOPMENT_SHADERS (COMPILE_SHADERS_FOR_DEVELOPMENT && PLATFORM_SUPPORTS_DEVELOPMENT_SHADERS)


// ---------------------------------------------------- Indirect parameter support

// sizeof(FRHIDispatchIndirectParametersNoPadding) / sizeof(uint)
#define DISPATCH_INDIRECT_NO_PADDING_UINT_COUNT		3

// sizeof(FRHIDispatchIndirectParameters) / sizeof(uint)
// The size of FRHIDispatchIndirectParameters may vary per-platform due to padding requirements.  Platforms which diverge
// from the default of 3 uints must define DISPATCH_INDIRECT_UINT_COUNT in their PlatformCommon.ush to override the default
// behavior implemented here.
#ifndef DISPATCH_INDIRECT_UINT_COUNT
	#define DISPATCH_INDIRECT_UINT_COUNT	DISPATCH_INDIRECT_NO_PADDING_UINT_COUNT

	void WriteDispatchIndirectArgs(RWBuffer<uint> RWIndirectDispatchArgsBuffer, in uint InIndex, in uint InIndirectArgX, in uint InIndirectArgY, in uint InIndirectArgZ)
	{
		RWIndirectDispatchArgsBuffer[DISPATCH_INDIRECT_UINT_COUNT * InIndex + 0] = InIndirectArgX;
		RWIndirectDispatchArgsBuffer[DISPATCH_INDIRECT_UINT_COUNT * InIndex + 1] = InIndirectArgY;
		RWIndirectDispatchArgsBuffer[DISPATCH_INDIRECT_UINT_COUNT * InIndex + 2] = InIndirectArgZ;
	}
#endif // #ifndef DISPATCH_INDIRECT_UINT_COUNT

void WriteDispatchIndirectArgs(RWBuffer<uint> RWIndirectDispatchArgsBuffer, in uint InIndex, in uint3 InIndirectArg)
{
	WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, InIndex, InIndirectArg.x, InIndirectArg.y, InIndirectArg.z);
}

// sizeof(FRHIDrawIndirectParameters) / sizeof(uint)
#define DRAW_INDIRECT_UINT_COUNT 4

void WriteIndirectDrawArgs(RWBuffer<uint> OutIndirectArgsBuffer, in uint InIndex, in uint InVertexCount, in uint InInstanceCount, in uint StartVertexLocation=0, in uint StartInstanceLocation=0)
{
	OutIndirectArgsBuffer[DRAW_INDIRECT_UINT_COUNT * InIndex + 0] = InVertexCount;
	OutIndirectArgsBuffer[DRAW_INDIRECT_UINT_COUNT * InIndex + 1] = InInstanceCount;
	OutIndirectArgsBuffer[DRAW_INDIRECT_UINT_COUNT * InIndex + 2] = StartVertexLocation;
	OutIndirectArgsBuffer[DRAW_INDIRECT_UINT_COUNT * InIndex + 3] = StartInstanceLocation;
}

// sizeof(FRHIDrawIndexedIndirectParameters) / sizeof(uint)
#define DRAW_INDEXED_INDIRECT_UINT_COUNT 5

void WriteIndirectIndexedDrawArgs(RWBuffer<uint> OutIndirectArgsBuffer, in uint InIndex, in uint InIndexCount, in uint InInstanceCount, in uint InStartIndexLocation=0, in uint InBaseVertexLocation=0, in uint InStartInstanceLocation=0)
{
	OutIndirectArgsBuffer[DRAW_INDEXED_INDIRECT_UINT_COUNT * InIndex + 0] = InIndexCount;
	OutIndirectArgsBuffer[DRAW_INDEXED_INDIRECT_UINT_COUNT * InIndex + 1] = InInstanceCount;
	OutIndirectArgsBuffer[DRAW_INDEXED_INDIRECT_UINT_COUNT * InIndex + 2] = InStartIndexLocation;
	OutIndirectArgsBuffer[DRAW_INDEXED_INDIRECT_UINT_COUNT * InIndex + 3] = InBaseVertexLocation;
	OutIndirectArgsBuffer[DRAW_INDEXED_INDIRECT_UINT_COUNT * InIndex + 4] = InStartInstanceLocation;
}

// ---------------------------------------------------- Compiler missing implementations

#if COMPILER_NEEDS_DETERMINANT

	float determinant(float3x3 M)
	{
		return
			M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) -
			M[1][0] * (M[0][1] * M[2][2] - M[0][2] * M[2][1]) +
			M[2][0] * (M[0][1] * M[1][2] - M[0][2] * M[1][1]);
	}

#endif

#if COMPILER_HLSLCC
#define log10(x)  log((x)) / log(10.0)
#endif


#if !COMPILER_SUPPORTS_MINMAX3

float min3(float a, float b, float c)
{
	return min(a, min(b, c));
}

int min3(int a, int b, int c)
{
	return min(a, min(b, c));
}

uint min3(uint a, uint b, uint c)
{
	return min(a, min(b, c));
}

DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(min3, float)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(min3, int)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(min3, uint)

float max3(float a, float b, float c)
{
	return max(a, max(b, c));
}

int max3(int a, int b, int c)
{
	return max(a, max(b, c));
}

uint max3(uint a, uint b, uint c)
{
	return max(a, max(b, c));
}

DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(max3, float)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(max3, int)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(max3, uint)

#if PLATFORM_SUPPORTS_REAL_TYPES

half min3(half a, half b, half c)
{
	return min(a, min(b, c));
}

int16_t min3(int16_t a, int16_t b, int16_t c)
{
	return min(a, min(b, c));
}

uint16_t min3(uint16_t a, uint16_t b, uint16_t c)
{
	return min(a, min(b, c));
}

DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(min3, half)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(min3, int16_t)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(min3, uint16_t)

half max3(half a, half b, half c)
{
	return max(a, max(b, c));
}

int16_t max3(int16_t a, int16_t b, int16_t c)
{
	return max(a, max(b, c));
}

uint16_t max3(uint16_t a, uint16_t b, uint16_t c)
{
	return max(a, max(b, c));
}

DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(max3, half)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(max3, int16_t)
DECLARE_VECTOR_FUNCTION_OVERLOAD_3_PARAM(max3, uint16_t)

#endif // PLATFORM_SUPPORTS_REAL_TYPES

#endif // !COMPILER_SUPPORTS_MINMAX3

// https://devblogs.microsoft.com/directx/announcing-hlsl-2021/
// HLSL 2021 supports Logical Operator Short Circuiting. To do vector bool operations, need to use and() or() select()
// Sadly the HLSL2021 standard does not overload select() very well...
#define select(cond,a,b) select_internal(cond,a,b)
#define DEFINE_SELECT(TYPE) \
	TYPE    select_internal(bool    c, TYPE    a, TYPE    b) { return TYPE   (c   ? a.x : b.x); } \
	\
	TYPE##2 select_internal(bool    c, TYPE    a, TYPE##2 b) { return TYPE##2(c   ? a   : b.x, c   ? a   : b.y); } \
	TYPE##2 select_internal(bool    c, TYPE##2 a, TYPE    b) { return TYPE##2(c   ? a.x : b  , c   ? a.y : b  ); } \
	TYPE##2 select_internal(bool    c, TYPE##2 a, TYPE##2 b) { return TYPE##2(c   ? a.x : b.x, c   ? a.y : b.y); } \
	TYPE##2 select_internal(bool2   c, TYPE    a, TYPE    b) { return TYPE##2(c.x ? a   : b  , c.y ? a   : b  ); } \
	TYPE##2 select_internal(bool2   c, TYPE    a, TYPE##2 b) { return TYPE##2(c.x ? a   : b.x, c.y ? a   : b.y); } \
	TYPE##2 select_internal(bool2   c, TYPE##2 a, TYPE    b) { return TYPE##2(c.x ? a.x : b  , c.y ? a.y : b  ); } \
	TYPE##2 select_internal(bool2   c, TYPE##2 a, TYPE##2 b) { return TYPE##2(c.x ? a.x : b.x, c.y ? a.y : b.y); } \
	\
	TYPE##3 select_internal(bool    c, TYPE    a, TYPE##3 b) { return TYPE##3(c   ? a   : b.x, c   ? a   : b.y, c   ? a   : b.z); } \
	TYPE##3 select_internal(bool    c, TYPE##3 a, TYPE    b) { return TYPE##3(c   ? a.x : b  , c   ? a.y : b  , c   ? a.z : b  ); } \
	TYPE##3 select_internal(bool    c, TYPE##3 a, TYPE##3 b) { return TYPE##3(c   ? a.x : b.x, c   ? a.y : b.y, c   ? a.z : b.z); } \
	TYPE##3 select_internal(bool3   c, TYPE    a, TYPE    b) { return TYPE##3(c.x ? a   : b  , c.y ? a   : b  , c.z ? a   : b  ); } \
	TYPE##3 select_internal(bool3   c, TYPE    a, TYPE##3 b) { return TYPE##3(c.x ? a   : b.x, c.y ? a   : b.y, c.z ? a   : b.z); } \
	TYPE##3 select_internal(bool3   c, TYPE##3 a, TYPE    b) { return TYPE##3(c.x ? a.x : b  , c.y ? a.y : b  , c.z ? a.z : b  ); } \
	TYPE##3 select_internal(bool3   c, TYPE##3 a, TYPE##3 b) { return TYPE##3(c.x ? a.x : b.x, c.y ? a.y : b.y, c.z ? a.z : b.z); } \
	\
	TYPE##4 select_internal(bool    c, TYPE    a, TYPE##4 b) { return TYPE##4(c   ? a   : b.x, c   ? a   : b.y, c   ? a   : b.z, c   ? a   : b.w); } \
	TYPE##4 select_internal(bool    c, TYPE##4 a, TYPE    b) { return TYPE##4(c   ? a.x : b  , c   ? a.y : b  , c   ? a.z : b  , c   ? a.w : b  ); } \
	TYPE##4 select_internal(bool    c, TYPE##4 a, TYPE##4 b) { return TYPE##4(c   ? a.x : b.x, c   ? a.y : b.y, c   ? a.z : b.z, c   ? a.w : b.w); } \
	TYPE##4 select_internal(bool4   c, TYPE    a, TYPE    b) { return TYPE##4(c.x ? a   : b  , c.y ? a   : b  , c.z ? a   : b  , c.w ? a   : b  ); } \
	TYPE##4 select_internal(bool4   c, TYPE    a, TYPE##4 b) { return TYPE##4(c.x ? a   : b.x, c.y ? a   : b.y, c.z ? a   : b.z, c.w ? a   : b.w); } \
	TYPE##4 select_internal(bool4   c, TYPE##4 a, TYPE    b) { return TYPE##4(c.x ? a.x : b  , c.y ? a.y : b  , c.z ? a.z : b  , c.w ? a.w : b  ); } \
	TYPE##4 select_internal(bool4   c, TYPE##4 a, TYPE##4 b) { return TYPE##4(c.x ? a.x : b.x, c.y ? a.y : b.y, c.z ? a.z : b.z, c.w ? a.w : b.w); } \

DEFINE_SELECT(bool)
#if !(COMPILER_HLSL || COMPILER_DXC || COMPILER_HLSLCC) || PLATFORM_NEEDS_SELECT_UINT
// @todo-lh: Ambiguous for DXC and HLSLcc if no suffix is provided for integer literals
DEFINE_SELECT(uint)
#endif
DEFINE_SELECT(int)
DEFINE_SELECT(float)
#if PLATFORM_SUPPORTS_REAL_TYPES
DEFINE_SELECT(half)
#if !(COMPILER_HLSL || COMPILER_DXC || COMPILER_HLSLCC)
// @todo-lh: Ambiguous for DXC and HLSLcc if no suffix is provided for integer literals
DEFINE_SELECT(uint16_t)
#endif
DEFINE_SELECT(int16_t)
#endif
#undef DEFINE_SELECT

// Works around bug in the spirv for the missing implementation of the and() and or() intrinsics.
bool  and_internal(bool  a, bool  b) { return bool(a && b); }
bool2 and_internal(bool2 a, bool2 b) { return bool2(a.x && b.x, a.y && b.y); }
bool3 and_internal(bool3 a, bool3 b) { return bool3(a.x && b.x, a.y && b.y, a.z && b.z); }
bool4 and_internal(bool4 a, bool4 b) { return bool4(a.x && b.x, a.y && b.y, a.z && b.z, a.w && b.w); }

bool  or_internal(bool  a, bool  b) { return bool(a || b); }
bool2 or_internal(bool2 a, bool2 b) { return bool2(a.x || b.x, a.y || b.y); }
bool3 or_internal(bool3 a, bool3 b) { return bool3(a.x || b.x, a.y || b.y, a.z || b.z); }
bool4 or_internal(bool4 a, bool4 b) { return bool4(a.x || b.x, a.y || b.y, a.z || b.z, a.w || b.w); }

#define and(a, b) and_internal(a, b)
#define or(a, b) or_internal(a, b)

#if PLATFORM_SUPPORTS_REAL_TYPES && !defined(COMPILER_SUPPORTS_PACK_B32_B16)

// Function that explicitly use RDNA's v_pack_b32_f16 on supported platform. Note that RDNA's documentation call this instruction v_pack_b32_f16
// but really is a v_pack_b32_b16.
half2 v_pack_b32_b16(half a, half b)
{
	return half2(a, b);
}

int16_t2 v_pack_b32_b16(int16_t a, int16_t b)
{
	return int16_t2(a, b);
}

uint16_t2 v_pack_b32_b16(uint16_t a, uint16_t b)
{
	return uint16_t2(a, b);
}

#endif

#if !defined(COMPILER_SUPPORTS_COND_MASK)

float  CondMask(bool Cond, float  Src0, float  Src1) { return Cond ? Src0 : Src1; }
float2 CondMask(bool Cond, float2 Src0, float2 Src1) { return Cond ? Src0 : Src1; }
float3 CondMask(bool Cond, float3 Src0, float3 Src1) { return Cond ? Src0 : Src1; }
float4 CondMask(bool Cond, float4 Src0, float4 Src1) { return Cond ? Src0 : Src1; }

int  CondMask(bool Cond, int  Src0, int  Src1) { return Cond ? Src0 : Src1; }
int2 CondMask(bool Cond, int2 Src0, int2 Src1) { return Cond ? Src0 : Src1; }
int3 CondMask(bool Cond, int3 Src0, int3 Src1) { return Cond ? Src0 : Src1; }
int4 CondMask(bool Cond, int4 Src0, int4 Src1) { return Cond ? Src0 : Src1; }

uint  CondMask(bool Cond, uint  Src0, uint  Src1) { return Cond ? Src0 : Src1; }
uint2 CondMask(bool Cond, uint2 Src0, uint2 Src1) { return Cond ? Src0 : Src1; }
uint3 CondMask(bool Cond, uint3 Src0, uint3 Src1) { return Cond ? Src0 : Src1; }
uint4 CondMask(bool Cond, uint4 Src0, uint4 Src1) { return Cond ? Src0 : Src1; }

#endif

#if !defined(COMPILER_SUPPORTS_UNPACKBYTEN)
float UnpackByte0(uint v) { return float(v & 0xff); }
float UnpackByte1(uint v) { return float((v >> 8) & 0xff); }
float UnpackByte2(uint v) { return float((v >> 16) & 0xff); }
float UnpackByte3(uint v) { return float(v >> 24); }
#endif // !COMPILER_SUPPORTS_UNPACKBYTEN

#if !defined(COMPILER_SUPPORTS_BITFIELD_INTRINSICS)
#define COMPILER_SUPPORTS_BITFIELD_INTRINSICS 0

// Software emulation using SM5/GCN semantics.
// Fast as long as shifts, sizes and offsets are compile-time constant.
// TODO: Should we consider weaker semantics to allow for a more efficient implementation in the dynamic case?

uint BitFieldInsertU32(uint Mask, uint Preserve, uint Enable)
{
	return (Preserve & Mask) | (Enable & ~Mask);
}

uint BitFieldExtractU32(uint Data, uint Size, uint Offset)
{
	// Shift amounts are implicitly &31 in HLSL, so they should be optimized away on most platforms
	// In GLSL shift amounts < 0 or >= word_size are undefined, so we better be explicit
	Size &= 31;
	Offset &= 31;
	return (Data >> Offset) & ((1u << Size) - 1u);
}

int BitFieldExtractI32(int Data, uint Size, uint Offset)
{
	Size &= 31u;
	Offset &= 31u;
	const uint Shift = (32u - Size) & 31u;
	const int Value = (Data >> Offset) & int((1u << Size) - 1u);
	return (Value << Shift) >> Shift;
}

uint BitFieldMaskU32(uint MaskWidth, uint MaskLocation)
{
	MaskWidth &= 31u;
	MaskLocation &= 31u;

	return ((1u << MaskWidth) - 1u) << MaskLocation;
}
#endif

#if !defined(COMPILER_SUPPORTS_24BIT_INTRINSICS)
#define COMPILER_SUPPORTS_24BIT_INTRINSICS 0

int MulI24(int I1, int I2)
{
	// D.i32 = S0.i24 * S1.i24
	return (I1 * I2);
}

uint MulU24(uint U1, uint U2)
{
	// D.u32 = S0.u24 * S1.u24
	return U1 * U2;
}

int MadI24(int I1, int I2, int I3)
{
	//  D.i = S0.i[23:0] * S1.i[23:0] + S2.i
	return I1 * I2 + I3;
}

uint MadU24(uint U1, uint U2, uint U3)
{
	// D.u = S0.u[23:0] * S1.u[23:0] + S2.u
	return U1 * U2 + U3;
}

#endif

uint Padding(uint Value, uint Pow2)
{
	return (Value + Pow2 - 1u) & ~(Pow2 - 1u);
}

uint CeilLog2(uint Value)
{
	return Value < 2u ? 0u : firstbithigh(Value - 1u) + 1u;
}

float BitFieldExtractFloat(uint Bits, uint Count, uint Offset)
{
	return BitFieldExtractU32(Bits, Count, Offset) / (float)BitFieldMaskU32(Count, 0u);
}

#if !defined(COMPILER_SUPPORTS_BITALIGN)
#define COMPILER_SUPPORTS_BITALIGN 0
uint BitAlignU32(uint High, uint Low, uint Shift)
{
	Shift &= 31u;

	uint Result = Low >> Shift;
	Result |= Shift > 0u ? (High << (32u - Shift)) : 0u;
	return Result;
}
#endif

#ifndef COMPILER_SUPPORTS_BYTEALIGN
#define COMPILER_SUPPORTS_BYTEALIGN 0
uint ByteAlignU32(uint High, uint Low, uint Shift)
{
	return BitAlignU32(High, Low, Shift * 8);
}
#endif // #ifndef COMPILER_SUPPORTS_BYTEALIGN

#if COMPILER_HLSLCC
#define ddx_fine(x) ddx(x)
#define ddy_fine(y) ddy(y)
#endif

#ifndef COMPILER_SUPPORTS_ULONG_TYPES

#define UlongType uint2

UlongType PackUlongType(uint2 Value)
{
	return Value;
}

uint2 UnpackUlongType(UlongType Value)
{
	return Value;
}

#endif

// Prefix sum of Bits masked to the bits lower than Index.
uint MaskedBitCount( uint2 Bits, uint Index )
{
	bool bLow = Index < 32;

	uint Mask = 1u << ( Index - ( bLow ? 0 : 32 ) );
	Mask -= 1;

	uint Offset;
	Offset  = countbits( Bits.x & ( bLow ? Mask : ~0u ) );
	Offset += countbits( Bits.y & ( bLow ? 0 : Mask ) );
	return Offset;
}

// Lock a critical region of code within a pixel shader and guarantees no concurrent execution for the same pixel
#ifndef RASTER_ORDERED_VIEW_LOCK
#define RASTER_ORDERED_VIEW_LOCK()
#endif

// Unlock a critical region of code within a pixel shader.
#ifndef RASTER_ORDERED_VIEW_UNLOCK
#define RASTER_ORDERED_VIEW_UNLOCK()
#endif

#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS

#ifndef COMPILER_SUPPORTS_TO_SCALAR_MEMORY
#define COMPILER_SUPPORTS_TO_SCALAR_MEMORY 1
#define ToScalarMemory(x) WaveReadLaneFirst(x)
#endif

#ifndef COMPILER_SUPPORTS_MASKED_BIT_COUNT
#define COMPILER_SUPPORTS_MASKED_BIT_COUNT 1
uint MaskedBitCount( uint2 Bits )
{
	return MaskedBitCount( Bits, WaveGetLaneIndex() );
}
#endif

#if COMPILER_DXC
uint2 WaveBallot( bool Expr )
{
	return WaveActiveBallot( Expr ).xy;
}
#endif

#ifndef WaveReadLaneLast
uint WaveGetActiveLaneIndexLast()
{
	uint4 ActiveMask = WaveActiveBallot(true);
	uint Value, Offset;
	if (ActiveMask.w)
	{
		Value = ActiveMask.w;
		Offset = 96;
	}
	else if (ActiveMask.z)
	{
		Value = ActiveMask.z;
		Offset = 64;
	}
	else if (ActiveMask.y)
	{
		Value = ActiveMask.y;
		Offset = 32;
	}
	else
	{
		Value = ActiveMask.x;
		Offset = 0;
	}
	// v_ffbh_u32 converts SGPR to VGPR on AMD, so force back to SGPR with WaveReadLaneFirst.
	return WaveReadLaneFirst(firstbithigh(Value)) + Offset;
}

#define WaveReadLaneLast(x) WaveReadLaneAt( x, WaveGetActiveLaneIndexLast() )
#endif

#endif // PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS

#if !defined(COMPILER_SUPPORTS_WAVE_INCLUSIVE_PREFIX_SUM)
#define COMPILER_SUPPORTS_WAVE_INCLUSIVE_PREFIX_SUM 0
#define WaveInclusivePrefixSum(x) (WavePrefixSum(x) + x)
#endif

// Give hint to compiler to move one value to scalar unit.
#if !defined(ToScalarMemory) && !defined(COMPILER_SUPPORTS_TO_SCALAR_MEMORY)
	#define ToScalarMemory(x) (x)
#endif

#if FEATURE_LEVEL < FEATURE_LEVEL_ES3_1 && !COMPILER_METAL
	// DX11 (feature levels >= 10) feature sets natively supports uints in shaders; we just use floats on other platforms.
	#define uint4	int4
#endif

#ifndef SNORM
#if COMPILER_HLSLCC
	#define SNORM 
	#define UNORM
#else
	#define SNORM snorm
	#define UNORM unorm
#endif
#endif

#ifndef INFINITE_FLOAT
	#if COMPILER_HLSLCC
		#define INFINITE_FLOAT 3.402823e+38
	#else
		#define INFINITE_FLOAT 1.#INF
	#endif
#endif

#ifndef RWTextureCube
#define RWTextureCube RWTexture2DArray
#endif

#ifndef PLATFORM_NEEDS_PRECISE_SHADOW_DEPTH
#define PLATFORM_NEEDS_PRECISE_SHADOW_DEPTH 0
#endif

// Little tools to help with packing scalars arrays
#ifndef CALC_SCALAR_ARRAY_SIZE
#define CALC_SCALAR_ARRAY_SIZE(ElementCount) ((ElementCount+3)/4)
#endif
#ifndef DECLARE_SCALAR_ARRAY
#define DECLARE_SCALAR_ARRAY(ScalarType, ScalarName, ElementCount) ScalarType##4 ScalarName[CALC_SCALAR_ARRAY_SIZE(ElementCount)]
#endif
#ifndef GET_SCALAR_ARRAY_ELEMENT
#define GET_SCALAR_ARRAY_ELEMENT(PackedArray, ElementIndex) PackedArray[(uint)ElementIndex>>2u][(uint)ElementIndex&3u]
#endif

// Specialization constant support
#ifndef UE_DECL
	#define UE_DECL(Name)
#endif
#ifndef UE_EVAL
	#if ENABLE_SPECIALIZATION_CONSTANTS
		#define UE_EVAL(Name) Name##_DEFAULT
	#else
		#define UE_EVAL(Name) Name
	#endif
#endif

// Used by platforms that need extra decorations in pixel shaders when used with mesh shaders in pipelines
#ifndef MESH_SHADING_PER_PRIMITIVE_ATTRIBUTE
	#define MESH_SHADING_PER_PRIMITIVE_ATTRIBUTE
#endif
#ifndef MESH_SHADING_COMPATIBLE_PS_ENTRY_POINT
	#define MESH_SHADING_COMPATIBLE_PS_ENTRY_POINT
#endif
