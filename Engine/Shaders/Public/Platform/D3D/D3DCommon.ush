// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
D3DCommon.ush: Common defines for the D3D shader compiler
=============================================================================*/

// Update this GUID to invalidate D3D shaders and force a shader recompilation for D3D RHIs.
#pragma message("UESHADERMETADATA_VERSION ACD85E2E-2A94-4B29-B94B-608B96CABB3C")

#include "/Engine/Shared/HLSLReservedSpaces.h"

// DXC needs local globallycoherent variables to correctly end up as globallycoherent
#define COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS 1

#if SM6_PROFILE
	#define COMPILER_SUPPORTS_WAVE_SIZE 1
	#define WAVESIZE(N) [WaveSize(N)]
#endif

#if PLATFORM_SUPPORTS_STATIC_SAMPLERS

// Static sampler table must match D3D12RootSignature.cpp
SamplerState D3DStaticPointWrappedSampler       : register(s0, space1000);
SamplerState D3DStaticPointClampedSampler       : register(s1, space1000);
SamplerState D3DStaticBilinearWrappedSampler    : register(s2, space1000);
SamplerState D3DStaticBilinearClampedSampler    : register(s3, space1000);
SamplerState D3DStaticTrilinearWrappedSampler   : register(s4, space1000);
SamplerState D3DStaticTrilinearClampedSampler   : register(s5, space1000);

#define GetGlobalSampler(Filter, WrapMode) D3DStatic##Filter##WrapMode##Sampler

#endif // PLATFORM_SUPPORTS_STATIC_SAMPLERS

// TODO: Add support for the UEDiagnosticBuffer to other shader types
#if PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER && (PIXELSHADER || COMPUTESHADER || RAYGENSHADER)

// Diagnostic buffer which will be automatically bound by the RHI as a root parameter
// Counterpart to FD3D12DiagnosticBuffer::FQueue.Lanes in D3D12Adapter.h
// Diagnostic message layout: [uint counter, uint message_id, uint4 payload] (6 DWORDs total)
RWStructuredBuffer<uint> UEDiagnosticBuffer : UE_HLSL_REGISTER(u, 0, UE_HLSL_SPACE_DIAGNOSTIC);

void UEReportAssertWithPayload(uint AssertID, uint4 Payload)
{
	// Counterpart to FD3D12DiagnosticBuffer::FQueue::MaxLanes in D3D12Adapter.h
	static const uint UEDiagnosticMaxLanes = 64;
	static const uint NumUints = 6;

	uint LaneIndex = WaveGetLaneIndex();
	if (LaneIndex < UEDiagnosticMaxLanes)
	{
		// Write some diagnostic data if we're first
		uint OldValue = 0;
		InterlockedAdd(UEDiagnosticBuffer[LaneIndex * NumUints], 1, OldValue);

		if (OldValue == 0)
		{
			UEDiagnosticBuffer[(LaneIndex * NumUints) + 1] = AssertID;
			UEDiagnosticBuffer[(LaneIndex * NumUints) + 2] = Payload.x;
			UEDiagnosticBuffer[(LaneIndex * NumUints) + 3] = Payload.y;
			UEDiagnosticBuffer[(LaneIndex * NumUints) + 4] = Payload.z;
			UEDiagnosticBuffer[(LaneIndex * NumUints) + 5] = Payload.w;
		}

		// Try to trigger a crash (timeout) using a deliberate infinite loop
		// TODO: use PLATFORM_BREAK
		for (;;)
		{
			InterlockedAdd(UEDiagnosticBuffer[LaneIndex * NumUints], 0, OldValue);
			if (OldValue == 0)
			{
				break;
			}
		}
	}
}

#define PLATFORM_BREAK() UEReportAssertWithPayload(0xCECC, uint4(__LINE__, 0, 0, 0))

#define PLATFORM_ASSERT(condition, assert_id)              { if (!(condition)) UEReportAssertWithPayload(assert_id, uint4(0, 0, 0, 0)); }
#define PLATFORM_ASSERT1(condition, assert_id, a)          { if (!(condition)) UEReportAssertWithPayload(assert_id, uint4(a, 0, 0, 0));  }
#define PLATFORM_ASSERT2(condition, assert_id, a, b)       { if (!(condition)) UEReportAssertWithPayload(assert_id, uint4(a, b, 0, 0));  }
#define PLATFORM_ASSERT3(condition, assert_id, a, b, c)    { if (!(condition)) UEReportAssertWithPayload(assert_id, uint4(a, b, c, 0));  }
#define PLATFORM_ASSERT4(condition, assert_id, a, b, c, d) { if (!(condition)) UEReportAssertWithPayload(assert_id, uint4(a, b, c, d));  }

#define check(condition) 									{ if (!(condition)) { UEReportAssertWithPayload(0xCECC, uint4(__LINE__, TEXT(__FILE__), TEXT(#condition), 0)); } }
#define checkf(condition, message) 							{ if (!(condition)) { UEReportAssertWithPayload(0xCECC, uint4(__LINE__, TEXT(__FILE__), message, 0)); } }

#endif // PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER

#if PLATFORM_SUPPORTS_MESH_SHADERS_TIER0
	#define MESH_SHADER_TRIANGLE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("triangle")]
	#define MESH_SHADER_LINE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("line")]

	#define AMPLIFICATION_SHADER_TRIANGLE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("triangle")]
	#define AMPLIFICATION_SHADER_LINE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("line")]

	#define MESH_SHADER_VERTEX_EXPORT(VertexType, InMaxVertexCount) out vertices VertexType OutVertices[InMaxVertexCount]
	#define MESH_SHADER_TRIANGLE_EXPORT(InMaxTriangleCount) out indices uint3 OutTriangles[InMaxTriangleCount]
	#define MESH_SHADER_LINE_EXPORT(InMaxLineCount) out indices uint2 OutLines[InMaxLineCount]

	#define MESH_SHADER_WRITE_VERTEX(InVertexIndex, Value) OutVertices[InVertexIndex] = Value;
	#define MESH_SHADER_WRITE_TRIANGLE(InTriangleIndex, Value) OutTriangles[InTriangleIndex] = Value;
	#define MESH_SHADER_WRITE_LINE(InLineIndex, Value) OutLines[InLineIndex] = Value;
#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS_TIER1
	#define MESH_SHADER_PRIMITIVE_EXPORT(PrimitiveType, InMaxPrimitiveCount) out primitives PrimitiveType OutPrimitives[InMaxPrimitiveCount]

	#define MESH_SHADER_WRITE_PRIMITIVE(InPrimitiveIndex, Value) OutPrimitives[InPrimitiveIndex] = Value;
#endif

// Workaround for a potential shader compiler bug. Some triangles may get incorrectly clipped in the ShadowDepths pass if the output z is not precise
#define PLATFORM_NEEDS_PRECISE_SHADOW_DEPTH 1

#if __SHADER_TARGET_MAJOR == 6 && __SHADER_TARGET_MINOR >= 6
	#define COMPILER_SUPPORTS_ULONG_TYPES 1
	#define COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS (1 && COMPILER_SUPPORTS_ULONG_TYPES)

	#if COMPILER_SUPPORTS_ULONG_TYPES
		#define UlongType uint64_t

		UlongType PackUlongType(uint2 Value)
		{
			return ((UlongType)Value.y << 32) | Value.x;
		}

		uint2 UnpackUlongType(UlongType Value)
		{
			return uint2(uint(Value), uint(Value >> 32));
		}
	#endif

	#if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
		void ImageInterlockedMaxUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
		{
			InterlockedMax(Dest[Coord], Value);
		}

		void ImageInterlockedAddUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
		{
			InterlockedAdd(Dest[Coord], Value);
		}

		void ImageInterlockedOrUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
		{
			InterlockedOr(Dest[Coord], Value);
		}
	#endif
#endif

#if __SHADER_TARGET_MAJOR >= 6 && __SHADER_TARGET_MINOR >= 5
	#define COMPILER_SUPPORTS_WAVE_MATCH_INTRINSICS 1
#endif

#if PLATFORM_SUPPORTS_BINDLESS
	#if (RAYCALLABLESHADER || RAYHITGROUPSHADER || RAYGENSHADER || RAYMISSSHADER)
		// Bindless index can be divergent in hit, miss and callable shaders
		#define UE_BINDLESS_INDEX(Index) NonUniformResourceIndex(Index)
	#else
		#define UE_BINDLESS_INDEX(Index) Index
	#endif

	#define GetSRVFromHeap(Type, Index) ResourceDescriptorHeap[UE_BINDLESS_INDEX(Index)]
	#define GetUAVFromHeap(Type, Index) ResourceDescriptorHeap[UE_BINDLESS_INDEX(Index)]
	#define GetCBVFromHeap(Type, Index) ResourceDescriptorHeap[UE_BINDLESS_INDEX(Index)]
	#define GetSamplerFromHeap(Type, Index) SamplerDescriptorHeap[UE_BINDLESS_INDEX(Index)]
#endif	
