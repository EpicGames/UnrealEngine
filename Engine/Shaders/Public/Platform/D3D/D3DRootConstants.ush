// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/HLSLReservedSpaces.h"

struct FUERootConstants
{
	uint4 Values;
};

ConstantBuffer<FUERootConstants> UERootConstants : UE_HLSL_REGISTER(b, 0, UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS);

uint4 GetAllRootConstants() { return UERootConstants.Values; }

uint GetRootConstant0() { return UERootConstants.Values.x; }
uint GetRootConstant1() { return UERootConstants.Values.y; }
uint GetRootConstant2() { return UERootConstants.Values.z; }
uint GetRootConstant3() { return UERootConstants.Values.w; }
