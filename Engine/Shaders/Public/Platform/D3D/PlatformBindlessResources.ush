// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_BINDLESS_ENABLED && (RAYCALLABLESHADER || RAYHITGROUPSHADER || RAYGENSHADER || RAYMISSSHADER || COMPUTESHADER)
	#define OVERRIDE_RAY_TRACING_HIT_GROUP_SYSTEM_RESOURCES 1
	#include "/Engine/Shared/RayTracingBuiltInResources.h"

	// Combine the common FHitGroupSystemRootConstants with bindless parameters
	struct FD3DHitGroupSystemParameters
	{
		FHitGroupSystemRootConstants RootConstants;

		uint BindlessHitGroupSystemIndexBuffer;
		uint BindlessHitGroupSystemVertexBuffer;
	};

	ConstantBuffer<FD3DHitGroupSystemParameters>	D3DHitGroupSystemParameters : UE_HLSL_REGISTER(b, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
	
	static const FHitGroupSystemRootConstants HitGroupSystemRootConstants = D3DHitGroupSystemParameters.RootConstants;

	// Will get filled in with the bindless index in D3DHitGroupSystemParameters.BindlessHitGroupSystemIndexBuffer
	ByteAddressBuffer HitGroupSystemIndexBuffer;

	// Will get filled in with the bindless index in D3DHitGroupSystemParameters.BindlessHitGroupSystemVertexBuffer
	ByteAddressBuffer HitGroupSystemVertexBuffer;
#endif
