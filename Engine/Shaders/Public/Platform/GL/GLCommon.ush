// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GLCommon.ush: Common GL shader code
=============================================================================*/

#pragma once

// Update this GUID to invalidate shader recompilation for only GL shaders
#pragma message("UESHADERMETADATA_VERSION CE7F533A-EFA7-4EA0-B2CF-15B195E67095")


