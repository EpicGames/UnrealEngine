// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GLSubpassSupport.usf: Vulkan specific subpass intrinsics 
=============================================================================*/
#pragma once

#if PIXELSHADER
#if COMPILER_DXC

	// Note: Do not rename subpass input attachment declarations, or adjust your changes in VulkanBackend.cpp (see VULKAN_SUBPASS_FETCH_VAR_W)

	[[vk::input_attachment_index(0)]]
	SubpassInput<float> GENERATED_SubpassDepthFetchAttachment;
	#define GLSubpassDepthFetch() GENERATED_SubpassDepthFetchAttachment.SubpassLoad( )

	[[vk::input_attachment_index(1)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment0;
	#define GLSubpassFetch0() GENERATED_SubpassFetchAttachment0.SubpassLoad()

	[[vk::input_attachment_index(2)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment1;
	#define GLSubpassFetch1() GENERATED_SubpassFetchAttachment1.SubpassLoad()

	[[vk::input_attachment_index(3)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment2;
	#define GLSubpassFetch2() GENERATED_SubpassFetchAttachment2.SubpassLoad()
	
	[[vk::input_attachment_index(4)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment3;
	#define GLSubpassFetch3() GENERATED_SubpassFetchAttachment3.SubpassLoad()
	
#else // COMPILER_DXC

	#define GLSubpassDepthFetch() 1.0

	#define GLSubpassFetch0() float4(1.0)

	#define GLSubpassFetch1() float4(1.0)

	#define GLSubpassFetch2() float4(1.0)
	
	#define GLSubpassFetch3() float4(1.0)

#endif // COMPILER_DXC
#endif // PIXELSHADER