// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalCommon.usf: Common Metal shader code
=============================================================================*/

#pragma once

// Update this GUID to invalidate shader recompilation for only Metal shaders
#pragma message("UESHADERMETADATA_VERSION D1B69FDA-FF60-4414-8915-01A07FC67FF4")

#if VERTEXSHADER && HULLSHADER
#ifndef TESSELLATIONSHADER
#define TESSELLATIONSHADER 1
#endif
#endif

#define STRONG_TYPE
#define StrongTypedBuffer Buffer

#if PLATFORM_SUPPORTS_STATIC_SAMPLERS

// Static sampler table must match MetalStaticSamplers.cpp
SamplerState MetalStaticPointWrappedSampler       : register(s0, space1000);
SamplerState MetalStaticPointClampedSampler       : register(s1, space1000);
SamplerState MetalStaticBilinearWrappedSampler    : register(s2, space1000);
SamplerState MetalStaticBilinearClampedSampler    : register(s3, space1000);
SamplerState MetalStaticTrilinearWrappedSampler   : register(s4, space1000);
SamplerState MetalStaticTrilinearClampedSampler   : register(s5, space1000);

#define GetGlobalSampler(Filter, WrapMode) MetalStatic##Filter##WrapMode##Sampler

#endif // PLATFORM_SUPPORTS_STATIC_SAMPLERS

#if METAL_SM6_PROFILE == 1
    #define PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS 1
#endif

#if IS_BASE_PASS && COMPUTE_SHADED && !defined(USE_FORCE_TEXTURE_MIP)
	#define USE_FORCE_TEXTURE_MIP 1
#endif

// Apple needs a bias because of a precision issue that can't guarantee SV_DepthLessEqual
// This is causing base pass to partially fail depth test when depth prepass use OUTPUT_PIXEL_DEPTH_OFFSET
#if METAL_SM5_PROFILE || METAL_SM6_PROFILE || METAL_MRT_PROFILE
	#define APPLE_DEPTH_BIAS_HACK 1
	#define APPLE_DEPTH_BIAS_VALUE 0.000001
#endif

/* Too many bugs to enable wave-intrinsics just yet, but we are getting there.
#if COMPUTESHADER || PIXELSHADER
#define COMPILER_SUPPORTS_WAVE_ONCE 1
#define COMPILER_SUPPORTS_WAVE_VOTE 1
#define COMPILER_SUPPORTS_WAVE_MINMAX 1
#define COMPILER_SUPPORTS_WAVE_BIT_ORAND 1
#endif
*/

#define COMPILER_SUPPORTS_MINMAX3 0

#if IOS && PIXELSHADER 
	// The load operation here will be patched by the MetalShaderCompiler,
	// do NOT change any character in the "SubpassFetch*" macro definition, including white spaces!
	
	// 4 components: RGBA_0
	Texture2D<float4> gl_LastFragDataRGBA_0;
	#define SubpassFetchRGBA_0() gl_LastFragDataRGBA_0.Load(uint3(0, 0, 0), 0)
	
	// 1 component: R_4
	Texture2D<float> gl_LastFragDataR_4;
	#define SubpassFetchR_4() gl_LastFragDataR_4.Load(uint3(0, 0, 0), 0)
	
	// 1 component: R_5
	Texture2D<float> gl_LastFragDataR_5;
	#define SubpassFetchR_5() gl_LastFragDataR_5.Load(uint3(0, 0, 0), 0)

	Texture2D<float> gl_LastFragDataR_1;
	#define SubpassFetchR_1() gl_LastFragDataR_1.Load(uint3(0, 0, 0), 0)
	//
	// Rest of SubpassFetch can be found in MetalSubpassSupport.ush
	//
	#if MOBILE_DEFERRED_SHADING
		#if MOBILE_EXTENDED_GBUFFER
			#define DepthbufferFetchES2() SubpassFetchR_5()
		#else
			#define DepthbufferFetchES2() SubpassFetchR_4()
		#endif
	#else
		#define DepthbufferFetchES2() SubpassFetchR_1()
	#endif
#else
	#define SubpassFetchRGBA_0() ((float4)0.0)
	#define SubpassFetchR_0() (0.0)
	#define SubpassFetchR_4() (0.0)
	#define SubpassFetchR_5() (0.0)
#endif //IOS && PIXELSHADER

#if PLATFORM_SUPPORTS_BINDLESS
	#define GetSRVFromHeap(Type, Index) ResourceDescriptorHeap[Index]
	#define GetUAVFromHeap(Type, Index) ResourceDescriptorHeap[Index]
	#define GetCBVFromHeap(Type, Index) ResourceDescriptorHeap[Index]
	#define GetSamplerFromHeap(Type, Index) SamplerDescriptorHeap[Index]
#endif

#if METAL_SM6_PROFILE == 1	
    #define COMPILER_SUPPORTS_ULONG_TYPES 1
    #define COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS 1

    #if COMPILER_SUPPORTS_ULONG_TYPES
        #define UlongType uint64_t

        UlongType PackUlongType(uint2 Value)
        {
            return ((UlongType)Value.y << 32) | Value.x;
        }

        uint2 UnpackUlongType(UlongType Value)
        {
            return uint2(uint(Value), uint(Value >> 32));
        }
    #endif

    #if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
        void ImageInterlockedMaxUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
        {
            InterlockedMax(Dest[Coord], Value);
        }

		void ImageInterlockedAddUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
		{
			InterlockedAdd(Dest[Coord], Value);
		}

		void ImageInterlockedOrUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
		{
			InterlockedOr(Dest[Coord], Value);
		}

    #endif
#endif

#if UE_BINDLESS_ENABLED && \
	((RAYCALLABLESHADER || RAYHITGROUPSHADER || RAYGENSHADER || RAYMISSSHADER) || \
	(defined(PLATFORM_SUPPORTS_INLINE_RAY_TRACING) && PLATFORM_SUPPORTS_INLINE_RAY_TRACING == 1))

	#define OVERRIDE_RAY_TRACING_HIT_GROUP_SYSTEM_RESOURCES 1
	#include "/Engine/Shared/RayTracingBuiltInResources.h"

	struct FMetalHitGroupSystemParameters
	{
		FHitGroupSystemRootConstants RootConstants;

		uint BindlessHitGroupSystemIndexBuffer;
		uint BindlessHitGroupSystemVertexBuffer;
	};

	ConstantBuffer<FMetalHitGroupSystemParameters>	MetalHitGroupSystemParameters : UE_HLSL_REGISTER(b, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);

	static const FHitGroupSystemRootConstants HitGroupSystemRootConstants = MetalHitGroupSystemParameters.RootConstants;

	// Will get filled in with the bindless index in MetalHitGroupSystemParameters.BindlessHitGroupSystemIndexBuffer
	ByteAddressBuffer HitGroupSystemIndexBuffer;

	// Will get filled in with the bindless index in MetalHitGroupSystemParameters.BindlessHitGroupSystemVertexBuffer
	ByteAddressBuffer HitGroupSystemVertexBuffer;

#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS_TIER0
    #define MESH_SHADER_TRIANGLE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("triangle")]
    #define MESH_SHADER_LINE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("line")]

    #define AMPLIFICATION_SHADER_TRIANGLE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("triangle")]
    #define AMPLIFICATION_SHADER_LINE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("line")]

    #define MESH_SHADER_VERTEX_EXPORT(VertexType, InMaxVertexCount) out vertices VertexType OutVertices[InMaxVertexCount]
    #define MESH_SHADER_TRIANGLE_EXPORT(InMaxTriangleCount) out indices uint3 OutTriangles[InMaxTriangleCount]
    #define MESH_SHADER_LINE_EXPORT(InMaxLineCount) out indices uint2 OutLines[InMaxLineCount]

    #define MESH_SHADER_WRITE_VERTEX(InVertexIndex, Value) OutVertices[InVertexIndex] = Value;
    #define MESH_SHADER_WRITE_TRIANGLE(InTriangleIndex, Value) OutTriangles[InTriangleIndex] = Value;
    #define MESH_SHADER_WRITE_LINE(InLineIndex, Value) OutLines[InLineIndex] = Value;
#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS_TIER1
    #define MESH_SHADER_PRIMITIVE_EXPORT(PrimitiveType, InMaxPrimitiveCount) out primitives PrimitiveType OutPrimitives[InMaxPrimitiveCount]

    #define MESH_SHADER_WRITE_PRIMITIVE(InPrimitiveIndex, Value) OutPrimitives[InPrimitiveIndex] = Value;
#endif
