// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MetalSubpassSupport.usf: Metal specific subpass intrinsics 
=============================================================================*/
#pragma once

#if IOS && PIXELSHADER 
	// The load operation here will be patched by the MetalShaderCompiler,
	// do NOT change any character in the "SubpassFetch*" macro definition, including white spaces!
	//
	// SubpassFetchRGBA_0 declaration can be found in MetalCommon.ush
	//
	
	// 4 components
	Texture2D<float4> gl_LastFragDataRGBA_1;
	#define SubpassFetchRGBA_1() gl_LastFragDataRGBA_1.Load(uint3(0, 0, 0), 0)

	Texture2D<float4> gl_LastFragDataRGBA_2;
	#define SubpassFetchRGBA_2() gl_LastFragDataRGBA_2.Load(uint3(0, 0, 0), 0)

	Texture2D<float4> gl_LastFragDataRGBA_3;
	#define SubpassFetchRGBA_3() gl_LastFragDataRGBA_3.Load(uint3(0, 0, 0), 0)

	Texture2D<float4> gl_LastFragDataRGBA_4;
	#define SubpassFetchRGBA_4() gl_LastFragDataRGBA_4.Load(uint3(0, 0, 0), 0)
	
	Texture2D<float4> gl_LastFragDataRGBA_5;
	#define SubpassFetchRGBA_5() gl_LastFragDataRGBA_5.Load(uint3(0, 0, 0), 0)

	Texture2D<float4> gl_LastFragDataRGBA_6;
	#define SubpassFetchRGBA_6() gl_LastFragDataRGBA_6.Load(uint3(0, 0, 0), 0)

	Texture2D<float4> gl_LastFragDataRGBA_7;
	#define SubpassFetchRGBA_7() gl_LastFragDataRGBA_7.Load(uint3(0, 0, 0), 0)

	// 1 component
	Texture2D<float> gl_LastFragDataR_0;
	#define SubpassFetchR_0() gl_LastFragDataR_0.Load(uint3(0, 0, 0), 0)

    // R_1 in MetalCommon.ush

	Texture2D<float> gl_LastFragDataR_2;
	#define SubpassFetchR_2() gl_LastFragDataR_2.Load(uint3(0, 0, 0), 0)

	Texture2D<float> gl_LastFragDataR_3;
	#define SubpassFetchR_3() gl_LastFragDataR_3.Load(uint3(0, 0, 0), 0)
    
	// R_4 and R_5 in MetalCommon.ush

	Texture2D<float> gl_LastFragDataR_6;
	#define SubpassFetchR_6() gl_LastFragDataR_6.Load(uint3(0, 0, 0), 0)

	Texture2D<float> gl_LastFragDataR_7;
	#define SubpassFetchR_7() gl_LastFragDataR_7.Load(uint3(0, 0, 0), 0)
#else
	#define SubpassFetchRGBA_1() ((float4)0.0)
	#define SubpassFetchRGBA_2() ((float4)0.0)
	#define SubpassFetchRGBA_3() ((float4)0.0)
	#define SubpassFetchRGBA_4() ((float4)0.0)
	#define SubpassFetchRGBA_5() ((float4)0.0)
	#define SubpassFetchRGBA_6() ((float4)0.0)
	#define SubpassFetchRGBA_7() ((float4)0.0)
	#define SubpassFetchR_0() (0.0)
	#define SubpassFetchR_1() (0.0)
	#define SubpassFetchR_2() (0.0)
	#define SubpassFetchR_3() (0.0)
	#define SubpassFetchR_6() (0.0)
	#define SubpassFetchR_7() (0.0)
#endif //IOS && PIXELSHADER 
