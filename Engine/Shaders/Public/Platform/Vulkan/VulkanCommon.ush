// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanCommon.ush: Common Vulkan shader code
=============================================================================*/

#pragma once

// Update this GUID to invalidate shader recompilation for only Vulkan shaders
#pragma message("UESHADERMETADATA_VERSION 2252475E-CF8A-45F3-85D7-E06B3EDCCCFC")

#define COMPILER_SUPPORTS_ULONG_TYPES 1
#define COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS (1 && COMPILER_SUPPORTS_ULONG_TYPES)

#if COMPILER_SUPPORTS_ULONG_TYPES

#define UlongType uint64_t

UlongType PackUlongType(uint2 Value)
{
	return ((UlongType)Value.y << 32) | Value.x;
}

uint2 UnpackUlongType(UlongType Value)
{
	return uint2(uint(Value), uint(Value >> 32));
}

#endif

#if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
void ImageInterlockedMaxUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
{
	InterlockedMax(Dest[Coord], Value);
}

void ImageInterlockedAddUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
{
	InterlockedAdd(Dest[Coord], Value);
}

void ImageInterlockedOrUInt64(RWTexture2D<UlongType> Dest, uint2 Coord, UlongType Value)
{
	InterlockedOr(Dest[Coord], Value);
}
#endif

#if UE_BINDLESS_ENABLED
	// While we use descriptor indexing, declare unbounded resource arrays
	#define VULKAN_HEAP_NAME_INNER(Prefix, Type) Prefix##Type
	#define VULKAN_HEAP_NAME(Prefix, Type) VULKAN_HEAP_NAME_INNER(Prefix, Type)
	#define VULKAN_SAMPLER_HEAP(Type) VULKAN_HEAP_NAME(BINDLESS_SAMPLER_ARRAY_PREFIX,Type)
	#define VULKAN_SRV_HEAP(Type) VULKAN_HEAP_NAME(BINDLESS_SRV_ARRAY_PREFIX,Type)
	#define VULKAN_UAV_HEAP(Type) VULKAN_HEAP_NAME(BINDLESS_UAV_ARRAY_PREFIX,Type)
	#define VULKAN_CBV_HEAP(Type) VULKAN_HEAP_NAME(BINDLESS_CBV_ARRAY_PREFIX,Type)

	#if (RAYCALLABLESHADER || RAYHITGROUPSHADER || RAYGENSHADER || RAYMISSSHADER)
		#define VULKAN_HEAP_ACCESS(BindlessIndex) NonUniformResourceIndex(BindlessIndex)
	#else
		#define VULKAN_HEAP_ACCESS(BindlessIndex) BindlessIndex
	#endif

	#define UE_BINDLESS_SRV_HEAP_DECL(Type) Type VULKAN_SRV_HEAP(Type)[];
	#define UE_BINDLESS_UAV_HEAP_DECL(Type) Type VULKAN_UAV_HEAP(Type)[];
	#define UE_BINDLESS_CBV_HEAP_DECL(Type) Type VULKAN_CBV_HEAP(Type)[];
	#define UE_BINDLESS_SAMPLER_HEAP_DECL(Type) Type VULKAN_SAMPLER_HEAP(Type)[];

	#define GetSRVFromHeap(Type, Index) VULKAN_SRV_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]
	#define GetUAVFromHeap(Type, Index) VULKAN_UAV_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]
	#define GetCBVFromHeap(Type, Index) VULKAN_CBV_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]
	#define GetSamplerFromHeap(Type, Index) VULKAN_SAMPLER_HEAP(Type)[VULKAN_HEAP_ACCESS(Index)]
#endif

#if VULKAN_SUPPORTS_SUBGROUP_SIZE_CONTROL
	#define COMPILER_SUPPORTS_WAVE_SIZE 1

	// NOTE: Wave size in Vulkan is supported at shader stage in the pipeline creation.
	// Create something unique we can read back from the shader, but that goes away on the rewrite
	COMPILER_DEFINE VULKAN_WAVESIZE(N)
	#define WAVESIZE(N) VULKAN_WAVESIZE(N)
#endif

#if UE_BINDLESS_ENABLED && (RAYCALLABLESHADER || RAYHITGROUPSHADER || RAYGENSHADER || RAYMISSSHADER)
	#define OVERRIDE_RAY_TRACING_HIT_GROUP_SYSTEM_RESOURCES 1
	#include "/Engine/Shared/RayTracingBuiltInResources.h"

	struct VulkanShaderRecordLooseParameterData
	{
		uint VulkanShaderRecordDummyGlobals;
	};

	// Combine the common FHitGroupSystemRootConstants with bindless parameters
	// :todo-jn: must match with VulkanRayTracing.h decl until we create a common header like RayTracingBuiltInResources.h
	struct FVulkanHitGroupSystemParameters
	{
		FHitGroupSystemRootConstants RootConstants;

		uint BindlessHitGroupSystemIndexBuffer;
		uint BindlessHitGroupSystemVertexBuffer;

		uint BindlessUniformBuffers[32];

		// Globals must be kept at the end of FVulkanHitGroupSystemParameters
		VulkanShaderRecordLooseParameterData Globals;
	};

	[[vk::shader_record_ext]] ConstantBuffer<FVulkanHitGroupSystemParameters> VulkanHitGroupSystemParameters;

	static const FHitGroupSystemRootConstants HitGroupSystemRootConstants = VulkanHitGroupSystemParameters.RootConstants;

	// Will get filled in with the bindless index in VulkanHitGroupSystemParameters.BindlessHitGroupSystemIndexBuffer
	ByteAddressBuffer HitGroupSystemIndexBuffer;
	// Will get filled in with the bindless index in VulkanHitGroupSystemParameters.BindlessHitGroupSystemVertexBuffer
	ByteAddressBuffer HitGroupSystemVertexBuffer;
#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS_TIER0
	#define MESH_SHADER_TRIANGLE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("triangle")]
	#define MESH_SHADER_LINE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("line")]

	#define AMPLIFICATION_SHADER_TRIANGLE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("triangle")]
	#define AMPLIFICATION_SHADER_LINE_ATTRIBUTES(InThreadsX) [numthreads(InThreadsX, 1, 1)][outputtopology("line")]

	#define MESH_SHADER_VERTEX_EXPORT(VertexType, InMaxVertexCount) out vertices VertexType OutVertices[InMaxVertexCount]
	#define MESH_SHADER_TRIANGLE_EXPORT(InMaxTriangleCount) out indices uint3 OutTriangles[InMaxTriangleCount]
	#define MESH_SHADER_LINE_EXPORT(InMaxLineCount) out indices uint2 OutLines[InMaxLineCount]

	#define MESH_SHADER_WRITE_VERTEX(InVertexIndex, Value) OutVertices[InVertexIndex] = Value;
	#define MESH_SHADER_WRITE_TRIANGLE(InTriangleIndex, Value) OutTriangles[InTriangleIndex] = Value;
	#define MESH_SHADER_WRITE_LINE(InLineIndex, Value) OutLines[InLineIndex] = Value;
#endif

#if PLATFORM_SUPPORTS_MESH_SHADERS_TIER1
	#define MESH_SHADER_PRIMITIVE_EXPORT(PrimitiveType, InMaxPrimitiveCount) out primitives PrimitiveType OutPrimitives[InMaxPrimitiveCount]

	#define MESH_SHADER_WRITE_PRIMITIVE(InPrimitiveIndex, Value) OutPrimitives[InPrimitiveIndex] = Value;
#endif

// Pixel shaders fed by a mesh shaders must have matching PerPrimitiveEXT (5271) decoration, which requires adding the MeshShadingEXT (5283) capability
#if PIXELSHADER
	#define MESH_SHADING_PER_PRIMITIVE_ATTRIBUTE [[vk::ext_decorate(5271)]]
	#define MESH_SHADING_COMPATIBLE_PS_ENTRY_POINT [[vk::ext_extension("SPV_EXT_mesh_shader")]] [[vk::ext_capability(5283)]]
#endif

// Types of Global Samplers (see Common.ush for types)
// Must match GetGlobalSamplerType() in SpirVShaderCompiler.inl
// and EGlobalSamplerType in VulkanShaderResources.h

#if UE_BINDLESS_ENABLED
	typedef SamplerState SafeTypeGlobalSamplers;
	SafeTypeGlobalSamplers VULKAN_SAMPLER_HEAP(GlobalSamplers)[];
	static const SamplerState VulkanGlobalPointClampedSampler     = VULKAN_SAMPLER_HEAP(GlobalSamplers)[0];
	static const SamplerState VulkanGlobalPointWrappedSampler     = VULKAN_SAMPLER_HEAP(GlobalSamplers)[1];
	static const SamplerState VulkanGlobalBilinearClampedSampler  = VULKAN_SAMPLER_HEAP(GlobalSamplers)[2];
	static const SamplerState VulkanGlobalBilinearWrappedSampler  = VULKAN_SAMPLER_HEAP(GlobalSamplers)[3];
	static const SamplerState VulkanGlobalTrilinearClampedSampler = VULKAN_SAMPLER_HEAP(GlobalSamplers)[4];
	static const SamplerState VulkanGlobalTrilinearWrappedSampler = VULKAN_SAMPLER_HEAP(GlobalSamplers)[5];
#else
	SamplerState VulkanGlobalPointWrappedSampler;
	SamplerState VulkanGlobalPointClampedSampler;
	SamplerState VulkanGlobalBilinearWrappedSampler;
	SamplerState VulkanGlobalBilinearClampedSampler;
	SamplerState VulkanGlobalTrilinearWrappedSampler;
	SamplerState VulkanGlobalTrilinearClampedSampler;
#endif

#define GetGlobalSampler(Filter, WrapMode) VulkanGlobal##Filter##WrapMode##Sampler


#if UE_BINDLESS_ENABLED && RAYGENSHADER
// Push constants are currently only used for ray tracing's static UBs (Scene, View, NaniteRayTracing, LumenHardwareRayTracing)
struct FVulkanPushConstants
{
	uint StaticUniformBufferBindings[8];  // keep in sync with FRHIShaderBindingLayout::MaxUniformBufferEntries
};
[[vk::push_constant]] ConstantBuffer<FVulkanPushConstants> VulkanPushConstants;
#endif

// Specialization constant support
#if PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS && ENABLE_SPECIALIZATION_CONSTANTS
#define UE_DECL(Name) [[vk::constant_id(Name##_INDEX)]] const int Name = 0
#define UE_EVAL(Name) Name
#endif
