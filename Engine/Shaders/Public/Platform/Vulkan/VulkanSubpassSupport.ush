// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VulkanSubpassSupport.usf: Vulkan specific subpass intrinsics 
=============================================================================*/
#pragma once

#if COMPILER_DXC && PIXELSHADER

	#ifdef SUBPASS_MSAA_SAMPLES
		#if SUBPASS_MSAA_SAMPLES > 1
			#define USE_MULTI_SAMPLE 1
		#endif
	#endif

	// Note: Do not rename subpass input attachment declarations, or adjust your changes in VulkanBackend.cpp (see VULKAN_SUBPASS_FETCH_VAR_W)

	[[vk::input_attachment_index(0)]]
	SubpassInput<float> GENERATED_SubpassDepthFetchAttachment;
	#define VulkanSubpassDepthFetch() GENERATED_SubpassDepthFetchAttachment.SubpassLoad( )

	[[vk::input_attachment_index(1)]]
	#ifndef USE_MULTI_SAMPLE
		SubpassInput<float4> GENERATED_SubpassFetchAttachment0;
	#else
		SubpassInputMS<float4> GENERATED_SubpassFetchAttachment0;
	#endif
	#define VulkanSubpassFetch0 GENERATED_SubpassFetchAttachment0.SubpassLoad

	[[vk::input_attachment_index(2)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment1;
	#define VulkanSubpassFetch1() GENERATED_SubpassFetchAttachment1.SubpassLoad()

	[[vk::input_attachment_index(3)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment2;
	#define VulkanSubpassFetch2() GENERATED_SubpassFetchAttachment2.SubpassLoad()

	[[vk::input_attachment_index(4)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment3;
	#define VulkanSubpassFetch3() GENERATED_SubpassFetchAttachment3.SubpassLoad()

	[[vk::input_attachment_index(5)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment4;
	#define VulkanSubpassFetch4() GENERATED_SubpassFetchAttachment4.SubpassLoad()

	[[vk::input_attachment_index(6)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment5;
	#define VulkanSubpassFetch5() GENERATED_SubpassFetchAttachment5.SubpassLoad()

	[[vk::input_attachment_index(7)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment6;
	#define VulkanSubpassFetch6() GENERATED_SubpassFetchAttachment6.SubpassLoad()

	[[vk::input_attachment_index(8)]]
	SubpassInput<float4> GENERATED_SubpassFetchAttachment7;
	#define VulkanSubpassFetch7() GENERATED_SubpassFetchAttachment7.SubpassLoad()

#endif // COMPILER_DXC && PIXELSHADER
