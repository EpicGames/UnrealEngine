// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_SUPPORTS_SHADER_ROOT_CONSTANTS
	#if OVERRIDE_ROOTCONSTANTS_USH
		#include "/Platform/Public/PlatformRootConstants.ush"
	#elif SM6_PROFILE || SM5_PROFILE
		#include "Platform/D3D/D3DRootConstants.ush"
	#endif
#else
	uint4 PassData; 

	uint4 GetAllRootConstants() { return PassData; }

	uint GetRootConstant0() { return PassData.x; }
	uint GetRootConstant1() { return PassData.y; }
	uint GetRootConstant2() { return PassData.z; }
	uint GetRootConstant3() { return PassData.w; }
#endif
