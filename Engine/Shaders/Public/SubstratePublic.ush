// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	SubstratePublic.ush: Public facing API for external users of Substrate.
=============================================================================*/

#include "/Engine/Private/Substrate/Substrate.ush"
#include "/Engine/Private/Substrate/SubstrateRead.ush"

#if SUBSTRATE_ENABLED 

#define SUBSTRATE_ALLOWS_GBUFFER_SAMPLING  ((NEEDS_SCENE_TEXTURES && !SCENE_TEXTURES_DISABLED)) && SHADING_PATH_DEFERRED

/** Public facing functions allowing external users access to some Substrate data. */

#ifndef SubstratePublicStruct
#error One must specify the SubstratePublicStruct uniform buffer to fetch data from when using SubstratePuclic.ush
#endif

struct FSubstrateGBuffer
{
	bool bIsValid;

	float3 DiffuseColor;
	float3 SpecularColor;
	float3 SubsurfaceColor;
	float3 BaseColor;
	float  Specular;
	float  Metallic;
	float3 WorldNormal;
	float  Opacity;
	float  Roughness;
	float  MaterialAO;
	float3 ShadingModelColor;
	float  ShadingModelID;
	float3 StoredBaseColor;
	float3 StoredSpecular;
	float3 WorldTangent;
	float  Anisotropy;
	bool   bIsFirstPerson;
};

FSubstrateGBuffer SubstratePublic_GetSubstrateGBufferFromFirstBSDF(uint2 PixelCoord, bool bGetNormalizedNormal = true)
{
	FSubstrateGBuffer Out = (FSubstrateGBuffer)0; 
	Out.bIsValid = false;

#if SUBSTRATE_ALLOWS_GBUFFER_SAMPLING
	#if SUBSTRATE_GBUFFER_FORMAT == 1 && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	FSubstrateAddressing SubstrateAddressing   = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), SubstratePublicStruct.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(SubstratePublicStruct.MaterialTextureArray, SubstrateAddressing, SubstratePublicStruct.TopLayerTexture);
	
	BRANCH
	if (SubstratePixelHeader.ClosureCount > 0)
	{
		const FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(SubstratePublicStruct.MaterialTextureArray, SubstratePublicStruct.FirstSliceStoringSubstrateSSSData, SubstrateAddressing.PixelCoords);

		const bool bSkipSSSMaterialOverride = true; // We do not want the material data to be affected by any override.
		FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(SubstratePublicStruct.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader, bSkipSSSMaterialOverride);
	
		Out.bIsValid			= true;
		Out.DiffuseColor		= SubstrateGetBSDFDiffuseColor(BSDF);
		Out.SpecularColor		= SubstrateGetBSDFSpecularF0(BSDF);
		Out.SubsurfaceColor		= SubstrateGetBSDFSubSurfaceColor(BSDF);
		Out.BaseColor			= SubstrateGetBSDFBaseColor(BSDF);
		Out.Specular			= SubstrateGetBSDFSpecular(BSDF);
		Out.Metallic			= SubstrateGetBSDFMetallic(BSDF);
		Out.WorldNormal			= SubstrateGetWorldNormal(SubstratePixelHeader, BSDF, SubstrateAddressing);
		Out.Roughness			= SubstrateGetBSDFRoughness(BSDF);
		Out.MaterialAO			= SubstrateGetAO(SubstratePixelHeader);
		Out.ShadingModelID		= SubstrateGetLegacyShadingModels(BSDF);
		Out.ShadingModelColor	= GetShadingModelColor(Out.ShadingModelID);
		Out.StoredBaseColor		= SubstrateGetBSDFBaseColor(BSDF);
		Out.StoredSpecular		= SubstrateGetBSDFSpecular(BSDF).rrr;
		Out.WorldTangent		= SubstrateGetWorldTangent(SubstratePixelHeader, BSDF, SubstrateAddressing);
		Out.Anisotropy			= SubstrateGetBSDFAnisotropy(BSDF);
		Out.bIsFirstPerson		= SubstratePixelHeader.IsFirstPerson();
		Out.Opacity				= SubstrateGetBSDFOpacity(BSDF);

		// Override opacity with SSSHeader value if needed
		const uint SSSType = SubstrateSubSurfaceHeaderGetSSSType(SSSHeader);
		if (SSSType != SSS_TYPE_NONE && SSSType != SSS_TYPE_TRANSMITTANCE_SHADOW)
		{
			Out.Opacity = SubstrateSubSurfaceHeaderGetOpacity(SSSHeader);
		}
	}
	else if (SubstratePixelHeader.State == 0)				// Only Unlit and non written pixels have a state of 0. Those pixels should show up as unlit to respect the legacy behavior.
	{		
		Out.bIsValid			= true;
		Out.ShadingModelID		= SHADINGMODELID_UNLIT;
		Out.ShadingModelColor	= GetShadingModelColor(Out.ShadingModelID);
	}
	#elif SUBSTRATE_GBUFFER_FORMAT == 0
	const float2 UV = (float2(PixelCoord) + 0.5) * View.BufferSizeAndInvSize.zw;
	const FGBufferData GBuffer = GetScreenSpaceData(UV, bGetNormalizedNormal).GBuffer;
	Out.bIsValid			= GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
	Out.DiffuseColor		= GBuffer.DiffuseColor;
	Out.SpecularColor		= GBuffer.SpecularColor;
	Out.SubsurfaceColor		= IsSubsurfaceModel(GBuffer.ShadingModelID) ? ExtractSubsurfaceColor(GBuffer) : GBuffer.CustomData.xyz;
	Out.BaseColor			= GBuffer.BaseColor;
	Out.Specular			= GBuffer.Specular;
	Out.Metallic			= GBuffer.Metallic;
	Out.WorldNormal			= GBuffer.WorldNormal;
	Out.Opacity				= GBuffer.CustomData.a;
	Out.Roughness			= GBuffer.Roughness;
	Out.MaterialAO			= GBuffer.GBufferAO;
	Out.ShadingModelColor	= GetShadingModelColor(GBuffer.ShadingModelID);
	Out.ShadingModelID		= GBuffer.ShadingModelID;
	Out.StoredBaseColor		= GBuffer.StoredBaseColor;
	Out.StoredSpecular		= GBuffer.StoredSpecular;
	Out.WorldTangent		= GBuffer.WorldTangent;
	Out.Anisotropy			= GBuffer.Anisotropy;
	Out.bIsFirstPerson		= IsFirstPerson(GBuffer);
	#endif
#endif
	return Out;
}

// Optimised data fetch from the TopLayer texture
float3 SubstratePublic_GetWorldNormal(uint2 PixelCoord)
{
	float3 Out = float3(0, 0, 1);
#if SUBSTRATE_ALLOWS_GBUFFER_SAMPLING
	#if SUBSTRATE_GBUFFER_FORMAT == 1 && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	Out = SubstrateUnpackTopLayerData(SubstratePublicStruct.TopLayerTexture.Load(uint3(PixelCoord, 0))).WorldNormal;
	#elif SUBSTRATE_GBUFFER_FORMAT == 0
	const float2 UV = (float2(PixelCoord) + 0.5) * View.BufferSizeAndInvSize.zw;
	Out = GetScreenSpaceData(UV).GBuffer.WorldNormal;
	#endif
#endif
	return Out;
}
float SubstratePublic_GetRoughness(uint2 PixelCoord)
{
	float Out = 0;
#if SUBSTRATE_ALLOWS_GBUFFER_SAMPLING
	#if SUBSTRATE_GBUFFER_FORMAT == 1 && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	Out = SubstrateUnpackTopLayerData(SubstratePublicStruct.TopLayerTexture.Load(uint3(PixelCoord, 0))).Roughness;
	#elif SUBSTRATE_GBUFFER_FORMAT == 0
	const float2 UV = (float2(PixelCoord) + 0.5) * View.BufferSizeAndInvSize.zw;
	Out = GetScreenSpaceData(UV).GBuffer.Roughness;
	#endif
#endif
	return Out;
}

#endif