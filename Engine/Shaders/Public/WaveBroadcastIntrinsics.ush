// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WaveBroadcastIntrinsics.ush: Exposes intrisics to perform broadcasting
	within lanes of a same wave.
=============================================================================*/
  
#pragma once

#include "Platform.ush"


#if COMPILER_SUPPORTS_WAVE_PERMUTE
	#define PLATFORM_SUPPORTS_WAVE_READ_AT 1
#elif FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	#define PLATFORM_SUPPORTS_WAVE_READ_AT 1
#else
	#define PLATFORM_SUPPORTS_WAVE_READ_AT 0
#endif

#if COMPILER_SUPPORTS_WAVE_SWIZZLE_GCN
	#define PLATFORM_SUPPORTS_WAVE_BROADCAST 1
#elif FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	#define PLATFORM_SUPPORTS_WAVE_BROADCAST 1
#else
	#define PLATFORM_SUPPORTS_WAVE_BROADCAST 0
#endif

#if defined(COMPILER_SUPPORTS_WAVE_SWIZZLE_RDNA) && COMPILER_SUPPORTS_WAVE_SWIZZLE_RDNA
	#define PLATFORM_SUPPORTS_WAVE_ROTATE 1
#elif FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	#define PLATFORM_SUPPORTS_WAVE_ROTATE 1
#else
	#define PLATFORM_SUPPORTS_WAVE_ROTATE 0
#endif


/** Different instruction available for WaveBroadcast() */
	#define WAVE_BROADCAST_NOP 0
	#define WAVE_BROADCAST_READ_LANE_AT 1
	#define WAVE_BROADCAST_READ_LDS_AT 2
	#define WAVE_BROADCAST_GCN_SWIZZLE 3
	#define WAVE_BROADCAST_ROTATE 4

/** Compile time structure to choose which broadcasting should be done. */
struct FWaveBroadcastSettings
{
	// Broadcast operations.
	uint Operation;
	
	// Lane index to read from
	uint SourceLaneIndex;
	
	// index of the thread.
	uint GroupThreadIndex;

	// Informations about the broadcast for GCN's ds_swizzle.
	uint SwizzleAnd;
	uint SwizzleOr;
	uint SwizzleXor;
	
	// Information about the broadcast for RDNA's ds_swizzle's rotate functionality
	int Rotate;
	uint RotateFixMask;
};

/** Group shared memory . */
#if defined(WAVE_BROADCAST_GROUPSIZE)
groupshared uint SharedBroadcastLDS[WAVE_BROADCAST_GROUPSIZE];
#endif

/** Returns the index of the source lane for a dest lane index. */
CALL_SITE_DEBUGLOC
uint GetWaveBroadcastSourceLaneIndex(const FWaveBroadcastSettings Settings, uint DestLaneIndex)
{
	if (0)
	{
		 return DestLaneIndex;
	}
	else if (Settings.Operation == WAVE_BROADCAST_READ_LANE_AT)
	{
		return Settings.SourceLaneIndex;
	}
	else if (Settings.Operation == WAVE_BROADCAST_READ_LDS_AT)
	{
		return Settings.SourceLaneIndex;
	}
	else if (Settings.Operation == WAVE_BROADCAST_GCN_SWIZZLE)
	{
		return (((DestLaneIndex & Settings.SwizzleAnd) | Settings.SwizzleOr) ^ Settings.SwizzleXor) % 32;
	}
	else if (Settings.Operation == WAVE_BROADCAST_ROTATE)
	{
		return (Settings.RotateFixMask & DestLaneIndex) | ((~Settings.RotateFixMask) & uint(DestLaneIndex + Settings.Rotate));
	}
	return DestLaneIndex;
}

/** Generic broadcast instruction. */
CALL_SITE_DEBUGLOC
uint WaveBroadcast(const FWaveBroadcastSettings Settings, uint x)
{
	if (0)
	{
		 return x;
	}
	
	#if PLATFORM_SUPPORTS_WAVE_READ_AT
	else if (Settings.Operation == WAVE_BROADCAST_READ_LANE_AT)
	{
		#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
		{
			return WaveReadLaneAt(x, Settings.SourceLaneIndex);
		}
		#else
			#error Unimplemented
		#endif
	}
	#endif

	#if defined(WAVE_BROADCAST_GROUPSIZE)
	else if (Settings.Operation == WAVE_BROADCAST_READ_LDS_AT)
	{
		GroupMemoryBarrierWithGroupSync();

		SharedBroadcastLDS[Settings.GroupThreadIndex] = x;
		
		GroupMemoryBarrierWithGroupSync();

		return SharedBroadcastLDS[Settings.SourceLaneIndex];
	}
	#endif

	#if PLATFORM_SUPPORTS_WAVE_BROADCAST
	else if (Settings.Operation == WAVE_BROADCAST_GCN_SWIZZLE)
	{
		#if COMPILER_SUPPORTS_WAVE_SWIZZLE_GCN
		{
			return WaveLaneSwizzleGCN(x, Settings.SwizzleAnd, Settings.SwizzleOr, Settings.SwizzleXor);
		}
		#elif FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
		{
			return WaveReadLaneAt(x, GetWaveBroadcastSourceLaneIndex(Settings, WaveGetLaneIndex()));
		}
		#else
			#error Unimplemented
		#endif
	}
	#endif

	#if PLATFORM_SUPPORTS_WAVE_ROTATE
	else if (Settings.Operation == WAVE_BROADCAST_ROTATE)
	{
		#if defined(COMPILER_SUPPORTS_WAVE_SWIZZLE_RDNA) && COMPILER_SUPPORTS_WAVE_SWIZZLE_RDNA
		{
			return WaveLaneRotateSwizzleRDNA(x, /* rotate_amount = */ Settings.Rotate, Settings.RotateFixMask & 0x1F);
		}
		#elif FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
		{
			return WaveReadLaneAt(x, GetWaveBroadcastSourceLaneIndex(Settings, WaveGetLaneIndex()));
		}
		#else
			#error Unimplemented
		#endif
	}
	#endif

	return x;
}

CALL_SITE_DEBUGLOC
uint2 WaveBroadcast(const FWaveBroadcastSettings Settings, uint2 v)
{
	return uint2(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y));
} 

CALL_SITE_DEBUGLOC
uint3 WaveBroadcast(const FWaveBroadcastSettings Settings, uint3 v)
{
	return uint3(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z));
} 

CALL_SITE_DEBUGLOC
uint4 WaveBroadcast(const FWaveBroadcastSettings Settings, uint4 v)
{
	return uint4(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z),
		WaveBroadcast(Settings, v.w));
} 


CALL_SITE_DEBUGLOC
int WaveBroadcast(const FWaveBroadcastSettings Settings, int x)
{
	return asint(WaveBroadcast(Settings, asuint(x)));
} 

CALL_SITE_DEBUGLOC
int2 WaveBroadcast(const FWaveBroadcastSettings Settings, int2 v)
{
	return int2(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y));
} 

CALL_SITE_DEBUGLOC
int3 WaveBroadcast(const FWaveBroadcastSettings Settings, int3 v)
{
	return int3(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z));
} 

CALL_SITE_DEBUGLOC
int4 WaveBroadcast(const FWaveBroadcastSettings Settings, int4 v)
{
	return int4(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z),
		WaveBroadcast(Settings, v.w));
} 


CALL_SITE_DEBUGLOC
float WaveBroadcast(const FWaveBroadcastSettings Settings, float x)
{
	return asfloat(WaveBroadcast(Settings, asuint(x)));
}

CALL_SITE_DEBUGLOC
float2 WaveBroadcast(const FWaveBroadcastSettings Settings, float2 v)
{
	return float2(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y));
} 

CALL_SITE_DEBUGLOC
float3 WaveBroadcast(const FWaveBroadcastSettings Settings, float3 v)
{
	return float3(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z));
}

CALL_SITE_DEBUGLOC
float4 WaveBroadcast(const FWaveBroadcastSettings Settings, float4 v)
{
	return float4(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z),
		WaveBroadcast(Settings, v.w));
} 


CALL_SITE_DEBUGLOC
bool WaveBroadcast(const FWaveBroadcastSettings Settings, bool x)
{
	return bool(WaveBroadcast(Settings, uint(x)));
}

CALL_SITE_DEBUGLOC
bool2 WaveBroadcast(const FWaveBroadcastSettings Settings, bool2 v)
{
	return bool2(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y));
} 

CALL_SITE_DEBUGLOC
bool3 WaveBroadcast(const FWaveBroadcastSettings Settings, bool3 v)
{
	return bool3(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z));
}

CALL_SITE_DEBUGLOC
bool4 WaveBroadcast(const FWaveBroadcastSettings Settings, bool4 v)
{
	return bool4(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y),
		WaveBroadcast(Settings, v.z),
		WaveBroadcast(Settings, v.w));
} 


#if PLATFORM_SUPPORTS_REAL_TYPES

CALL_SITE_DEBUGLOC
uint16_t WaveBroadcast(const FWaveBroadcastSettings Settings, uint16_t x)
{
	return uint16_t(WaveBroadcast(Settings, uint(x)));
}

CALL_SITE_DEBUGLOC
uint16_t2 WaveBroadcast(const FWaveBroadcastSettings Settings, uint16_t2 v)
#if COMPILER_SUPPORT_UINT16_BITCAST
{
	uint vu = bit_cast_uint(v);
	uint bvu = WaveBroadcast(Settings, vu);
	return bit_cast_uint16_t2(bvu);
} 
#elif 1
{
	uint vu = uint(v.x) | (uint(v.y) << 16u);
	uint bvu = WaveBroadcast(Settings, vu);
	return uint16_t2(bvu, bvu >> 16u);
}
#else
{
	return uint16_t2(
		WaveBroadcast(Settings, v.x),
		WaveBroadcast(Settings, v.y));
}
#endif

CALL_SITE_DEBUGLOC
uint16_t3 WaveBroadcast(const FWaveBroadcastSettings Settings, uint16_t3 v)
{
	return uint16_t3(
		WaveBroadcast(Settings, v.xy),
		WaveBroadcast(Settings, v.z));
}

CALL_SITE_DEBUGLOC
uint16_t4 WaveBroadcast(const FWaveBroadcastSettings Settings, uint16_t4 v)
{
	return uint16_t4(
		WaveBroadcast(Settings, v.xy),
		WaveBroadcast(Settings, v.zw));
} 


CALL_SITE_DEBUGLOC
int16_t WaveBroadcast(const FWaveBroadcastSettings Settings, int16_t x)
{
	return asint16(WaveBroadcast(Settings, asuint16(x)));
}

CALL_SITE_DEBUGLOC
int16_t2 WaveBroadcast(const FWaveBroadcastSettings Settings, int16_t2 v)
{
	return asint16(WaveBroadcast(Settings, asuint16(v)));
}

CALL_SITE_DEBUGLOC
int16_t3 WaveBroadcast(const FWaveBroadcastSettings Settings, int16_t3 v)
{
	return asint16(WaveBroadcast(Settings, asuint16(v)));
}

CALL_SITE_DEBUGLOC
int16_t4 WaveBroadcast(const FWaveBroadcastSettings Settings, int16_t4 v)
{
	return asint16(WaveBroadcast(Settings, asuint16(v)));
}


CALL_SITE_DEBUGLOC
half WaveBroadcast(const FWaveBroadcastSettings Settings, half x)
{
	return asfloat16(WaveBroadcast(Settings, asuint16(x)));
}

CALL_SITE_DEBUGLOC
half2 WaveBroadcast(const FWaveBroadcastSettings Settings, half2 v)
{
	return asfloat16(WaveBroadcast(Settings, asuint16(v)));
}

CALL_SITE_DEBUGLOC
half3 WaveBroadcast(const FWaveBroadcastSettings Settings, half3 v)
{
	return asfloat16(WaveBroadcast(Settings, asuint16(v)));
}

CALL_SITE_DEBUGLOC
half4 WaveBroadcast(const FWaveBroadcastSettings Settings, half4 v)
{
	return asfloat16(WaveBroadcast(Settings, asuint16(v)));
}

#endif // PLATFORM_SUPPORTS_REAL_TYPES



/** Init a nop broadcast */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitNopBroadcast()
{
	FWaveBroadcastSettings Settings;
	Settings.Operation  = WAVE_BROADCAST_NOP;
	Settings.SourceLaneIndex  = 0;
	Settings.GroupThreadIndex = 0;
	Settings.SwizzleAnd = 0x00;
	Settings.SwizzleOr  = 0x00;
	Settings.SwizzleXor = 0x00;
	Settings.Rotate        = +0;
	Settings.RotateFixMask = 0x00;
	return Settings;
}


/**  Dynamically read another lane's register
 *
 *	Requires PLATFORM_SUPPORTS_WAVE_READ_AT.
 *
 *	return src[SourceLaneIndex];
 *
 *  Note the SourceLaneIndex can be dynamic according to SM 6.0's WaveReadLaneAt()
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitWaveReadLaneAt(uint SourceLaneIndex)
{
	FWaveBroadcastSettings Settings = InitNopBroadcast();
	Settings.Operation  = WAVE_BROADCAST_READ_LANE_AT;
	Settings.SourceLaneIndex = SourceLaneIndex;
	return Settings;
}

/**  Dynamically read another group thread's register through LDS
 *
 *	Requires setting WAVE_BROADCAST_GROUPSIZE to allocate groupshared memory.
 *
 *	LDS[GroupThreadIndex] = src;
 *	return LDS[SourceLaneIndex];
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitWaveReadLDS(uint SourceLaneIndex, uint GroupThreadIndex)
{
	FWaveBroadcastSettings Settings = InitNopBroadcast();
	Settings.Operation  = WAVE_BROADCAST_READ_LDS_AT;
	Settings.SourceLaneIndex = SourceLaneIndex;
	Settings.GroupThreadIndex  = GroupThreadIndex;
	return Settings;
}

/**  Converts a broadcast operation to be performed in SM5.
 *
 *	Requires setting WAVE_BROADCAST_GROUPSIZE to allocate groupshared memory.
 *
 *	Returns the exact sames data as Settings would otherwise return, but through LDS. Example:
 *
 *		const FWaveBroadcastSettings BroadcastSettings = InitWaveReadLaneAt(SourceLaneIndex);
 *
 *		#if PLATFORM_SUPPORTS_WAVE_READ_AT
 *			uint Desc = WaveBroadcast(BroadcastSettings, Src);
 *		#else
 *			uint Desc = WaveBroadcast(ConvertWaveBroadcastToLDS(BroadcastSettings, GroupThreadIndex), Src);
 *		#endif
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings ConvertWaveBroadcastToLDS(const FWaveBroadcastSettings Settings, uint GroupThreadIndex)
{
	return InitWaveReadLDS(GetWaveBroadcastSourceLaneIndex(Settings, /* DestLaneIndex = */ GroupThreadIndex), GroupThreadIndex);
}

/**  Broadcast in butterfly pattern.
 *
 *	Requires PLATFORM_SUPPORTS_WAVE_BROADCAST.
 *
 *	return src[laneId ^ XorButterFly];
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitWaveXorButterfly(const uint XorButterFly)
{
	FWaveBroadcastSettings Settings = InitNopBroadcast();
	Settings.Operation  = WAVE_BROADCAST_GCN_SWIZZLE;
	Settings.SwizzleAnd = 0x1F;
	Settings.SwizzleOr  = 0x00;
	Settings.SwizzleXor = XorButterFly;
	return Settings;
}

/** Swap left lane with righ lanes within lane group (size is power of two in [2; 64]). 
 *
 *	Requires PLATFORM_SUPPORTS_WAVE_BROADCAST.
 *
 *  If a lane is not active, the VGPR value returned is 0.
 *
 *  LaneGroupSize = 8
 *  LaneId = 1
 *
 *         |  lane group (size=8)  |
 *     x = | 0  1  2  3| 4  5  6  7| 8  9 ...
 *
 *  return | 4  5  6  7| 0  1  2  3|12 13 ...
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitWaveSwapWithinLaneGroup(const uint LaneGroupSize)
{
	return InitWaveXorButterfly(/* XorButterFly = */ LaneGroupSize >> 1);
}

/**  Broadcast inner lane group over a lane group (size is power of two in [2; 64]).
 *
 *	Requires PLATFORM_SUPPORTS_WAVE_BROADCAST.
 *
 *  If a lane is not active, the VGPR value returned is 0.
 *
 *  LaneGroupSize = 8
 *  InnerLaneGroupSize = 2
 *  InnerLaneGroupId = 1
 *
 *         |  lane group (size=8)  |
 *     x = | 0  1  2  3  4  5  6  7| 8  9 ...
 *
 *  return | 2  3  2  3  2  3  2  3|10 11 ...
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitWaveBroadcastLaneGroup(const uint LaneGroupSize, const uint InnerLaneGroupSize, const uint InnerLaneGroupId)
{
	const uint InnerGroupCount = LaneGroupSize / InnerLaneGroupSize;
	
	FWaveBroadcastSettings Settings = InitNopBroadcast();
	Settings.Operation  = WAVE_BROADCAST_GCN_SWIZZLE;
	Settings.SwizzleAnd = ~((InnerGroupCount - 1) * InnerLaneGroupSize);
	Settings.SwizzleOr  = InnerLaneGroupId * InnerLaneGroupSize;
	Settings.SwizzleXor = 0x00;
	return Settings;
}

/**  Rotate lane (-127 to +127) over a lane group (size is power of two in [2; 128]).
 *
 *  Notes: RDNA can only do this with LaneGroupSize <= 32
 *
 *	Requires PLATFORM_SUPPORTS_WAVE_ROTATE.
 *
 *  If a lane is not active, the VGPR value returned is 0.
 *
 *  LaneGroupSize = 8
 *  LaneRotation = +3
 *
 *         |  lane group (size=8)  |
 *     x = | 0  1  2  3  4  5  6  7| 8  9 ...
 *
 *  return | 3  4  5  6  7  0  1  2|11 12 ...
 */
CALL_SITE_DEBUGLOC
FWaveBroadcastSettings InitWaveRotateLaneGroup(const uint LaneGroupSize, const int LaneRotation)
{
	FWaveBroadcastSettings Settings;
	Settings.Operation     = WAVE_BROADCAST_ROTATE;
	Settings.Rotate        = LaneRotation;
	Settings.RotateFixMask = ~(LaneGroupSize - 1);
	return Settings;
}
